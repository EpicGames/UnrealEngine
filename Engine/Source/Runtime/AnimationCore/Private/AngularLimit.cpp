// Copyright Epic Games, Inc. All Rights Reserved.

#include "AngularLimit.h"

namespace AnimationCore
{
	bool ConstrainAngularRangeUsingEuler(FQuat& InOutQuatRotation, const FQuat& InRefRotation, const FVector& InLimitMinDegrees, const FVector& InLimitMaxDegrees) 
	{
		// Simple clamping of euler angles. This might be better off refactored to use switch/twist decomposition and maybe an ellipsoid clamp
		FQuat DeltaQuat = InRefRotation * InOutQuatRotation.Inverse();
		FRotator DeltaRotator = DeltaQuat.Rotator();
		FRotator NewRotator;
		NewRotator.Pitch = FMath::Clamp(DeltaRotator.Pitch, InLimitMinDegrees.Y, InLimitMaxDegrees.Y);
		NewRotator.Yaw = FMath::Clamp(DeltaRotator.Yaw, InLimitMinDegrees.X, InLimitMaxDegrees.X);
		NewRotator.Roll = FMath::Clamp(DeltaRotator.Roll, InLimitMinDegrees.Z, InLimitMaxDegrees.Z);
		DeltaQuat = FQuat(NewRotator);

		InOutQuatRotation = DeltaQuat.Inverse() * InRefRotation;

		return !NewRotator.Equals(DeltaRotator);
	}
}
