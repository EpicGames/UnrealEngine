// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationCore.h: Render core module implementation.
=============================================================================*/

#include "AnimationCore.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, AnimationCore);

DEFINE_LOG_CATEGORY(LogAnimationCore);
