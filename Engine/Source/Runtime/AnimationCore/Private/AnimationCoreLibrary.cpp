// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationCore.h: Render core module implementation.
=============================================================================*/

#include "AnimationCoreLibrary.h"
#include "Constraint.h"
#include "AnimationCoreUtil.h"

namespace AnimationCore
{

static void AccumulateConstraintTransform(const FTransform& TargetTransform, const FConstraintDescription& Operator, float Weight, FComponentBlendHelper& BlendHelper)
{
	// now filter by operation
	if (Operator.bParent)
	{
		BlendHelper.AddParent(TargetTransform, Weight);
	}
	else
	{
		if (Operator.bTranslation)
		{
			FVector Translation = TargetTransform.GetTranslation();
			Operator.TranslationAxes.FilterVector(Translation);
			BlendHelper.AddTranslation(Translation, Weight);
		}

		if (Operator.bRotation)
		{
			FQuat DeltaRotation = TargetTransform.GetRotation();
			Operator.RotationAxes.FilterQuat(DeltaRotation);
			BlendHelper.AddRotation(DeltaRotation, Weight);
		}

		if (Operator.bScale)
		{
			FVector Scale = TargetTransform.GetScale3D();
			Operator.ScaleAxes.FilterVector(Scale);
			BlendHelper.AddScale(Scale, Weight);
		}
	}
}

FTransform SolveConstraints(const FTransform& CurrentTransform, const FTransform& BaseTransform, const TArray<struct FTransformConstraint>& Constraints, const FGetGlobalTransform& OnGetGlobalTransform)
{
	int32 TotalNum = Constraints.Num();
	ensureAlways(TotalNum > 0);
	check(OnGetGlobalTransform.IsBound());

	FComponentBlendHelper BlendHelper;
	FTransform BlendedTransform = CurrentTransform;

	for (int32 ConstraintIndex = 0; ConstraintIndex < TotalNum; ++ConstraintIndex)
	{
		const FTransformConstraint& Constraint = Constraints[ConstraintIndex];

		if (Constraint.Weight > ZERO_ANIMWEIGHT_THRESH)
		{
			// constraint has to happen in relative to parent to keep the hierarchy data
			// we'd like to test if this would work well with rotation
			FTransform ConstraintTransform = OnGetGlobalTransform.Execute(Constraint.TargetNode);
			FTransform ConstraintToParent = ConstraintTransform.GetRelativeTransform(BaseTransform);
			AccumulateConstraintTransform(ConstraintToParent, Constraint.Operator, Constraint.Weight, BlendHelper);
		}
	}

	// @note : parent and any other combination of constraints won't work
	FTransform ParentTransform;
	if (BlendHelper.GetBlendedParent(ParentTransform))
	{
		BlendedTransform = ParentTransform;
	}
	else
	{
		FVector BlendedTranslation;
		if (BlendHelper.GetBlendedTranslation(BlendedTranslation))
		{
			// if any result
			BlendedTransform.SetTranslation(BlendedTranslation);
		}
		FQuat BlendedRotation;
		if (BlendHelper.GetBlendedRotation(BlendedRotation))
		{
			BlendedTransform.SetRotation(BlendedRotation);
		}
		FVector BlendedScale;
		if (BlendHelper.GetBlendedScale(BlendedScale))
		{
			BlendedTransform.SetScale3D(BlendedScale);
		}
	}

	return BlendedTransform;
}

FQuat SolveAim(const FTransform& CurrentTransform, const FVector& TargetPosition, const FVector& AimVector, bool bUseUpVector /*= false*/, const FVector& UpVector /*= FVector::UpVector*/, float AimClampInDegree /*= 0.f*/)
{
	if (!ensure(AimVector.IsNormalized()) || !ensure(!bUseUpVector || UpVector.IsNormalized()))
	{
		return FQuat::Identity;
	}

	FTransform NewTransform = CurrentTransform;
	FVector ToTarget = TargetPosition - NewTransform.GetLocation();
	ToTarget.Normalize();

	if (AimClampInDegree > ZERO_ANIMWEIGHT_THRESH)
	{
		double AimClampInRadians = FMath::DegreesToRadians(FMath::Min(AimClampInDegree, 180.0));
		double DiffAngle = FMath::Acos(FVector::DotProduct(AimVector, ToTarget));

		if (DiffAngle > AimClampInRadians)
		{
			check(DiffAngle > 0.0);

			FVector DeltaTarget = ToTarget - AimVector;
			// clamp delta target to within the ratio
			DeltaTarget *= (AimClampInRadians / DiffAngle);
			// set new target
			ToTarget = AimVector + DeltaTarget;
			ToTarget.Normalize();
		}
	}

	// if want to use look up, project to the plane
	if (bUseUpVector)
	{
		// project target to the plane
		ToTarget = FVector::VectorPlaneProject(ToTarget, UpVector);
		ToTarget.Normalize();
	}

	return FQuat::FindBetweenNormals(AimVector, ToTarget);
}

///////////////////////////////////////////////////////////////
// new constraints

FTransform SolveConstraints(const FTransform& CurrentTransform, const FTransform& CurrentParentTransform, const TArray<struct FConstraintData>& Constraints)
{
	int32 TotalNum = Constraints.Num();
	ensure(TotalNum > 0);

	FMultiTransformBlendHelper BlendHelperInLocalSpace;
	FTransform BlendedLocalTransform = CurrentTransform.GetRelativeTransform(CurrentParentTransform);

	for (int32 ConstraintIndex = 0; ConstraintIndex < TotalNum; ++ConstraintIndex)
	{
		const FConstraintData& Constraint = Constraints[ConstraintIndex];

		if (Constraint.Weight > ZERO_ANIMWEIGHT_THRESH)
		{
			// constraint has to happen in relative to parent to keep the hierarchy data
			// we'd like to test if this would work well with rotation
			FTransform ConstraintTransform = Constraint.CurrentTransform;
			Constraint.ApplyConstraintTransform(ConstraintTransform, CurrentTransform, CurrentParentTransform, BlendHelperInLocalSpace);
		}
	}

	// @note : parent and any other combination of constraints won't work
	FTransform ParentTransform;
	if (BlendHelperInLocalSpace.GetBlendedParent(ParentTransform))
	{
		BlendedLocalTransform = ParentTransform;
	}
	else
	{
		FVector BlendedTranslation;
		if (BlendHelperInLocalSpace.GetBlendedTranslation(BlendedTranslation))
		{
			// if any result
			BlendedLocalTransform.SetTranslation(BlendedTranslation);
		}
		FQuat BlendedRotation;
		if (BlendHelperInLocalSpace.GetBlendedRotation(BlendedRotation))
		{
			BlendedLocalTransform.SetRotation(BlendedRotation);
		}
		FVector BlendedScale;
		if (BlendHelperInLocalSpace.GetBlendedScale(BlendedScale))
		{
			BlendedLocalTransform.SetScale3D(BlendedScale);
		}
	}

	return BlendedLocalTransform * CurrentParentTransform;
}

FQuat QuatFromEuler(const FVector& XYZAnglesInDegrees, EEulerRotationOrder RotationOrder, bool bUseUEHandyness)
{
	double X = FMath::DegreesToRadians(XYZAnglesInDegrees.X);
	double Y = FMath::DegreesToRadians(XYZAnglesInDegrees.Y);
	double Z = FMath::DegreesToRadians(XYZAnglesInDegrees.Z);

	if (bUseUEHandyness)
	{
		X = -X;
		Y = -Y;
	}

	double CosX = FMath::Cos( X * 0.5 );
	double CosY = FMath::Cos( Y * 0.5 );
	double CosZ = FMath::Cos( Z * 0.5 );

	double SinX = FMath::Sin( X * 0.5 );
	double SinY = FMath::Sin( Y * 0.5 );
	double SinZ = FMath::Sin( Z * 0.5 );

	if ( RotationOrder == EEulerRotationOrder::XYZ )
	{
		return FQuat(SinX * CosY * CosZ - CosX * SinY * SinZ, 
					 CosX * SinY * CosZ + SinX * CosY * SinZ,
					 CosX * CosY * SinZ - SinX * SinY * CosZ,
					 CosX * CosY * CosZ + SinX * SinY * SinZ);

	}
	else if ( RotationOrder == EEulerRotationOrder::XZY )
	{
		return FQuat(SinX * CosY * CosZ + CosX * SinY * SinZ, 
					 CosX * SinY * CosZ + SinX * CosY * SinZ,
					 CosX * CosY * SinZ - SinX * SinY * CosZ,
					 CosX * CosY * CosZ - SinX * SinY * SinZ);

	}
	else if ( RotationOrder == EEulerRotationOrder::YXZ )
	{
		return FQuat(SinX * CosY * CosZ - CosX * SinY * SinZ, 
					 CosX * SinY * CosZ + SinX * CosY * SinZ,
					 CosX * CosY * SinZ + SinX * SinY * CosZ,
					 CosX * CosY * CosZ - SinX * SinY * SinZ);

	}
	else if ( RotationOrder == EEulerRotationOrder::YZX )
	{
		return FQuat(SinX * CosY * CosZ - CosX * SinY * SinZ, 
					 CosX * SinY * CosZ - SinX * CosY * SinZ,
					 CosX * CosY * SinZ + SinX * SinY * CosZ,
					 CosX * CosY * CosZ + SinX * SinY * SinZ);
	}
	else if ( RotationOrder == EEulerRotationOrder::ZXY )
	{
		return FQuat(SinX * CosY * CosZ + CosX * SinY * SinZ, 
					 CosX * SinY * CosZ - SinX * CosY * SinZ,
					 CosX * CosY * SinZ - SinX * SinY * CosZ,
					 CosX * CosY * CosZ + SinX * SinY * SinZ);

	}
	else if ( RotationOrder == EEulerRotationOrder::ZYX )
	{
		return FQuat(SinX * CosY * CosZ + CosX * SinY * SinZ, 
					 CosX * SinY * CosZ - SinX * CosY * SinZ,
					 CosX * CosY * SinZ + SinX * SinY * CosZ,
					 CosX * CosY * CosZ - SinX * SinY * SinZ);

	}

	// should not happen
	return FQuat::Identity;
}

FVector EulerFromQuat(const FQuat& Rotation, EEulerRotationOrder RotationOrder, bool bUseUEHandyness)
{
	double X = Rotation.X;
	double Y = Rotation.Y;
	double Z = Rotation.Z;
	double W = Rotation.W;
	double X2 = X * 2.0;
	double Y2 = Y * 2.0;
	double Z2 = Z * 2.0;
	double XX2 = X * X2;
	double XY2 = X * Y2;
	double XZ2 = X * Z2;
	double YX2 = Y * X2;
	double YY2 = Y * Y2;
	double YZ2 = Y * Z2;
	double ZX2 = Z * X2;
	double ZY2 = Z * Y2;
	double ZZ2 = Z * Z2;
	double WX2 = W * X2;
	double WY2 = W * Y2;
	double WZ2 = W * Z2;

	FVector AxisX, AxisY, AxisZ;
	AxisX.X = (1.0 - (YY2 + ZZ2));
	AxisY.X = (XY2 + WZ2);
	AxisZ.X = (XZ2 - WY2);
	AxisX.Y = (XY2 - WZ2);
	AxisY.Y = (1.0 - (XX2 + ZZ2));
	AxisZ.Y = (YZ2 + WX2);
	AxisX.Z = (XZ2 + WY2);
	AxisY.Z = (YZ2 - WX2);
	AxisZ.Z = (1.0 - (XX2 + YY2));

	FVector Result = FVector::ZeroVector;

	if ( RotationOrder == EEulerRotationOrder::XYZ )
	{
		Result.Y = FMath::Asin( - FMath::Clamp( AxisZ.X, -1.0, 1.0 ) );

		if ( FMath::Abs( AxisZ.X ) < 1.0 - DOUBLE_KINDA_SMALL_NUMBER)
		{
			Result.X = FMath::Atan2( AxisZ.Y, AxisZ.Z );
			Result.Z = FMath::Atan2( AxisY.X, AxisX.X );
		}
		else
		{
			Result.X = 0.0;
			Result.Z = FMath::Atan2( -AxisX.Y, AxisY.Y );
		}
	}
	else if ( RotationOrder == EEulerRotationOrder::XZY )
	{
		// using KINDA_SMALL_NUMBER instead of SMALL_NUMBER so that it
		// better covers singularity cases, a specific case that was causing problems
		// was Quat(0.122787803968.., -0.122787803968.., -0.696364240320.., 0.696364240320..)
		Result.Z = FMath::Asin( FMath::Clamp( AxisY.X, -1.0, 1.0 ) );

		if ( FMath::Abs( AxisY.X ) < 1.0 - DOUBLE_KINDA_SMALL_NUMBER)
		{
			Result.X = FMath::Atan2( -AxisY.Z, AxisY.Y );
			Result.Y = FMath::Atan2( -AxisZ.X, AxisX.X );
		}
		else
		{
			Result.X = 0.0;
			Result.Y = FMath::Atan2( AxisX.Z, AxisZ.Z );
		}
	}
	else if ( RotationOrder == EEulerRotationOrder::YXZ )
	{
		Result.X = FMath::Asin( FMath::Clamp( AxisZ.Y, -1.0, 1.0 ) );

		if ( FMath::Abs( AxisZ.Y ) < 1.0 - DOUBLE_KINDA_SMALL_NUMBER)
		{
			Result.Y = FMath::Atan2( -AxisZ.X, AxisZ.Z );
			Result.Z = FMath::Atan2( -AxisX.Y, AxisY.Y );
		}
		else
		{
			Result.Y = 0.0;
			Result.Z = FMath::Atan2( AxisY.X, AxisX.X );
		}
	}
	else if ( RotationOrder == EEulerRotationOrder::YZX )
	{
		Result.Z = FMath::Asin( - FMath::Clamp( AxisX.Y, -1.0, 1.0 ) );

		if ( FMath::Abs( AxisX.Y ) < 1.0 - DOUBLE_KINDA_SMALL_NUMBER)
		{
			Result.X = FMath::Atan2( AxisZ.Y, AxisY.Y );
			Result.Y = FMath::Atan2( AxisX.Z, AxisX.X );
		}
		else
		{
			Result.X = FMath::Atan2( -AxisY.Z, AxisZ.Z );
			Result.Y = 0.0;
		}
	}
	else if ( RotationOrder == EEulerRotationOrder::ZXY )
	{
		Result.X = FMath::Asin( - FMath::Clamp( AxisY.Z, -1.0, 1.0 ) );

		if ( FMath::Abs( AxisY.Z ) < 1.0 - DOUBLE_KINDA_SMALL_NUMBER)
		{
			Result.Y = FMath::Atan2( AxisX.Z, AxisZ.Z );
			Result.Z = FMath::Atan2( AxisY.X, AxisY.Y );
		}
		else
		{
			Result.Y = FMath::Atan2( -AxisZ.X, AxisX.X );
			Result.Z = 0.0;
		}
	}
	else if ( RotationOrder == EEulerRotationOrder::ZYX )
	{
		Result.Y = FMath::Asin( FMath::Clamp( AxisX.Z, -1.0, 1.0 ) );

		if ( FMath::Abs( AxisX.Z ) < 1.0 - DOUBLE_KINDA_SMALL_NUMBER)
		{
			Result.X = FMath::Atan2( -AxisY.Z, AxisZ.Z );
			Result.Z = FMath::Atan2( -AxisX.Y, AxisX.X );
		}
		else
		{
			Result.X = FMath::Atan2( AxisZ.Y, AxisY.Y );
			Result.Z = 0.0;
		}
	}
	if (bUseUEHandyness)
	{
		Result.X = -Result.X;
		Result.Y = -Result.Y;
	}

	return Result * 180.0 / DOUBLE_PI;
}

FVector ChangeEulerRotationOrder(const FVector& XYZAnglesInDegrees, EEulerRotationOrder SourceRotationOrder, EEulerRotationOrder TargetRotationOrder, bool bUseUEHandyness)
{
	if(SourceRotationOrder == TargetRotationOrder)
	{
		return XYZAnglesInDegrees;
	}
	
	const FQuat Quaternion = QuatFromEuler(XYZAnglesInDegrees, SourceRotationOrder, bUseUEHandyness);
	return EulerFromQuat(Quaternion, TargetRotationOrder, bUseUEHandyness);

}

}