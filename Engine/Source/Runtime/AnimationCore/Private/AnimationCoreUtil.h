// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationCoreUtil.h: Render core module definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

struct FComponentBlendHelper
{
	void Reset()
	{
		Transforms.Reset();
		Translations.Reset();
		Rotations.Reset();
		Scales.Reset();
		TranslationWeights.Reset();
		RotationWeights.Reset();
		ScaleWeights.Reset();
	}

	void AddParent(const FTransform& InTransform, float Weight)
	{
		Transforms.Add(InTransform);
		ParentWeights.Add(Weight);
		ensureAlways(Transforms.Num() == ParentWeights.Num());
	}

	void AddTranslation(const FVector& Translation, float Weight)
	{
		Translations.Add(Translation);
		TranslationWeights.Add(Weight);
		ensureAlways(Translations.Num() == TranslationWeights.Num());
	}

	void AddRotation(const FQuat& Rotation, float Weight)
	{
		Rotations.Add(Rotation);
		RotationWeights.Add(Weight);
		ensureAlways(Rotations.Num() == RotationWeights.Num());
	}

	void AddScale(const FVector& Scale, float Weight)
	{
		Scales.Add(Scale);
		ScaleWeights.Add(Weight);
		ensureAlways(Scales.Num() == ScaleWeights.Num());
	}

	bool GetBlendedParent(FTransform& OutTransform)
	{
		// there is no correct value to return if no translation
		// so if false, do not use this value
		if (Transforms.Num() > 0)
		{
			float TotalWeight = GetTotalWeight(ParentWeights);

			if (TotalWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				float MultiplyWeight = (TotalWeight > 1.f) ? 1.f / TotalWeight : 1.f;

				int32 NumBlends = Transforms.Num();

				float ParentWeight = ParentWeights[0] * MultiplyWeight;
				FVector		OutTranslation = Transforms[0].GetTranslation() *  ParentWeight;
				FQuat		OutRotation = Transforms[0].GetRotation() * ParentWeight;
				FVector		OutScale = Transforms[0].GetScale3D() * ParentWeight;

				// otherwise we just purely blend by number, and then later we normalize
				for (int32 Index = 1; Index < NumBlends; ++Index)
				{
					// Simple linear interpolation for translation and scale.
					ParentWeight = ParentWeights[Index] * MultiplyWeight;
					OutTranslation = FMath::Lerp(OutTranslation, Transforms[Index].GetTranslation(), ParentWeight);
					OutScale = OutScale + Transforms[Index].GetScale3D()*ParentWeight;
					OutRotation = FQuat::FastLerp(OutRotation, Transforms[Index].GetRotation(), ParentWeight);
				}

				OutRotation.Normalize();
				OutTransform = FTransform(OutRotation, OutTranslation, OutScale);

				return true;
			}
		}

		return false;
	}

	bool GetBlendedTranslation(FVector& Output)
	{
		// there is no correct value to return if no translation
		// so if false, do not use this value
		if (Translations.Num() > 0)
		{
			float TotalWeight = GetTotalWeight(TranslationWeights);

			if (TotalWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				float MultiplyWeight = (TotalWeight > 1.f) ? 1.f / TotalWeight : 1.f;

				Output = Translations[0] * (TranslationWeights[0] * MultiplyWeight);

				for (int32 Index = 1; Index < Translations.Num(); ++Index)
				{
					Output += Translations[Index] * (TranslationWeights[Index] * MultiplyWeight);
				}

				return true;
			}
		}

		return false;
	}

	bool GetBlendedRotation(FQuat& Output)
	{
		// there is no correct value to return if no translation
		// so if false, do not use this value
		if (Rotations.Num() > 0)
		{
			float TotalWeight = GetTotalWeight(RotationWeights);

			if (TotalWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				float MultiplyWeight = (TotalWeight > 1.f) ? 1.f / TotalWeight : 1.f;

				Output = Rotations[0] * (RotationWeights[0] * MultiplyWeight);

				for (int32 Index = 1; Index < Rotations.Num(); ++Index)
				{
					FQuat BlendRotation = Rotations[Index] * (RotationWeights[Index] * MultiplyWeight);
					if ((Output | BlendRotation) < 0)
					{
						Output.X -= BlendRotation.X;
						Output.Y -= BlendRotation.Y;
						Output.Z -= BlendRotation.Z;
						Output.W -= BlendRotation.W;
					}
					else
					{
						Output.X += BlendRotation.X;
						Output.Y += BlendRotation.Y;
						Output.Z += BlendRotation.Z;
						Output.W += BlendRotation.W;
					}
				}

				Output.Normalize();
				return true;
			}
		}

		return false;
	}
	bool GetBlendedScale(FVector& Output)
	{
		// there is no correct value to return if no translation
		// so if false, do not use this value
		if (Scales.Num() > 0)
		{
			float TotalWeight = GetTotalWeight(ScaleWeights);

			if (TotalWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				float MultiplyWeight = (TotalWeight > 1.f) ? 1.f / TotalWeight : 1.f;

				Output = FVector::ZeroVector;

				for (int32 Index = 0; Index < Scales.Num(); ++Index)
				{
					Output += Scales[Index] * (ScaleWeights[Index] * MultiplyWeight);
				}

				return true;
			}
		}

		return false;
	}

private:
	// data member to accumulate blending intermediate result per component
	TArray<FTransform>	Transforms;
	TArray<FVector>		Translations;
	TArray<FQuat>		Rotations;
	TArray<FVector>		Scales;
	TArray<float>		ParentWeights;
	TArray<float>		TranslationWeights;
	TArray<float>		RotationWeights;
	TArray<float>		ScaleWeights;

	float GetTotalWeight(const TArray<float>& Weights)
	{
		float TotalWeight = 0.f;
		for (float Weight : Weights)
		{
			TotalWeight += Weight;
		}

		return TotalWeight;
	}
};

struct FMultiTransformBlendHelper
{
	void Reset()
	{
		Transforms.Reset();
		Translations.Reset();
		Rotations.Reset();
		Scales.Reset();
		TranslationWeights.Reset();
		RotationWeights.Reset();
		ScaleWeights.Reset();
	}

	void AddParent(const FTransform& InTransform, float Weight)
	{
		Transforms.Add(InTransform);
		ParentWeights.Add(Weight);
		ensureAlways(Transforms.Num() == ParentWeights.Num());
	}

	void AddTranslation(const FVector& Translation, float Weight)
	{
		Translations.Add(Translation);
		TranslationWeights.Add(Weight);
		ensureAlways(Translations.Num() == TranslationWeights.Num());
	}

	void AddRotation(const FQuat& Rotation, float Weight)
	{
		Rotations.Add(Rotation);
		RotationWeights.Add(Weight);
		ensureAlways(Rotations.Num() == RotationWeights.Num());
	}

	void AddScale(const FVector& Scale, float Weight)
	{
		Scales.Add(Scale);
		ScaleWeights.Add(Weight);
		ensureAlways(Scales.Num() == ScaleWeights.Num());
	}

	bool GetBlendedParent(FTransform& OutTransform)
	{
		// there is no correct value to return if no translation
		// so if false, do not use this value
		if (Transforms.Num() > 0)
		{
			float TotalWeight = GetTotalWeight(ParentWeights);

			if (TotalWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				float MultiplyWeight = (TotalWeight > 1.f) ? 1.f / TotalWeight : 1.f;

				int32 NumBlends = Transforms.Num();

				OutTransform = Transforms[0] * ScalarRegister(ParentWeights[0] * MultiplyWeight);

				// otherwise we just purely blend by number, and then later we normalize
				for (int32 Index = 1; Index < NumBlends; ++Index)
				{
					// Simple linear interpolation for translation and scale.
					OutTransform.AccumulateWithShortestRotation(Transforms[Index], ScalarRegister(ParentWeights[Index] * MultiplyWeight));
				}

				OutTransform.NormalizeRotation();

				return true;
			}
		}

		return false;
	}

	bool GetBlendedTranslation(FVector& Output)
	{
		// there is no correct value to return if no translation
		// so if false, do not use this value
		if (Translations.Num() > 0)
		{
			float TotalWeight = GetTotalWeight(TranslationWeights);

			if (TotalWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				float MultiplyWeight = (TotalWeight > 1.f) ? 1.f / TotalWeight : 1.f;

				Output = Translations[0] * (TranslationWeights[0] * MultiplyWeight);

				for (int32 Index = 1; Index < Translations.Num(); ++Index)
				{
					Output += Translations[Index] * (TranslationWeights[Index] * MultiplyWeight);
				}

				return true;
			}
		}

		return false;
	}

	bool GetBlendedRotation(FQuat& Output)
	{
		// there is no correct value to return if no translation
		// so if false, do not use this value
		if (Rotations.Num() > 0)
		{
			float TotalWeight = GetTotalWeight(RotationWeights);

			if (TotalWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				float MultiplyWeight = (TotalWeight > 1.f) ? 1.f / TotalWeight : 1.f;

				Output = Rotations[0] * (RotationWeights[0] * MultiplyWeight);

				for (int32 Index = 1; Index < Rotations.Num(); ++Index)
				{
					FQuat BlendRotation = Rotations[Index] * (RotationWeights[Index] * MultiplyWeight);
					if ((Output | BlendRotation) < 0)
					{
						Output.X -= BlendRotation.X;
						Output.Y -= BlendRotation.Y;
						Output.Z -= BlendRotation.Z;
						Output.W -= BlendRotation.W;
					}
					else
					{
						Output.X += BlendRotation.X;
						Output.Y += BlendRotation.Y;
						Output.Z += BlendRotation.Z;
						Output.W += BlendRotation.W;
					}
				}

				Output.Normalize();
				return true;
			}
		}

		return false;
	}
	bool GetBlendedScale(FVector& Output)
	{
		// there is no correct value to return if no translation
		// so if false, do not use this value
		if (Scales.Num() > 0)
		{
			float TotalWeight = GetTotalWeight(ScaleWeights);

			if (TotalWeight > ZERO_ANIMWEIGHT_THRESH)
			{
				float MultiplyWeight = (TotalWeight > 1.f) ? 1.f / TotalWeight : 1.f;

				Output = FVector::ZeroVector;

				for (int32 Index = 0; Index < Scales.Num(); ++Index)
				{
					Output += Scales[Index] * (ScaleWeights[Index] * MultiplyWeight);
				}

				return true;
			}
		}

		return false;
	}

private:
	// data member to accumulate blending intermediate result per component
	TArray<FTransform>	Transforms;
	TArray<FVector>		Translations;
	TArray<FQuat>		Rotations;
	TArray<FVector>		Scales;
	TArray<float>		ParentWeights;
	TArray<float>		TranslationWeights;
	TArray<float>		RotationWeights;
	TArray<float>		ScaleWeights;

	float GetTotalWeight(const TArray<float>& Weights)
	{
		float TotalWeight = 0.f;
		for (float Weight : Weights)
		{
			TotalWeight += Weight;
		}

		return TotalWeight;
	}
};
