// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationDataSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationDataSource)

bool UAnimationDataSourceRegistry::RegisterDataSource(const FName& InName, UObject* InDataSource)
{
	ClearInvalidDataSource();

	if (DataSources.Contains(InName))
	{
		return false;
	}
	DataSources.Add(InName, InDataSource);
	return true;
}

bool UAnimationDataSourceRegistry::UnregisterDataSource(const FName& InName)
{
	ClearInvalidDataSource();
	return DataSources.Remove(InName) > 0;
}

bool UAnimationDataSourceRegistry::ContainsSource(const FName& InName) const
{
	return DataSources.Contains(InName);
}

UObject* UAnimationDataSourceRegistry::RequestSource(const FName& InName, UClass* InExpectedClass) const
{
	TWeakObjectPtr<UObject> const* DataSource = DataSources.Find(InName);
	if (DataSource == nullptr)
	{
		return nullptr;
	}

	UObject* DataSourcePtr = DataSource->Get();
	if (!DataSourcePtr)
	{
		return nullptr;
	}
	if (!DataSourcePtr->IsA(InExpectedClass))
	{
		return nullptr;
	}

	return DataSourcePtr;
}

void UAnimationDataSourceRegistry::ClearInvalidDataSource()
{
	TArray<FName> InvalidNames;
	for (auto Iter = DataSources.CreateIterator(); Iter; ++Iter)
	{
		if (!Iter.Value().IsValid())
		{
			InvalidNames.Add(Iter.Key());
		}
	}

	for (const FName& NameToRemove : InvalidNames)
	{
		DataSources.Remove(NameToRemove);
	}
}
