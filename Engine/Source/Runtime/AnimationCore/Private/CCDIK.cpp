// Copyright Epic Games, Inc. All Rights Reserved.

#include "CCDIK.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CCDIK)

namespace AnimationCore
{

	bool SolveCCDIK(TArray<FCCDIKChainLink>& InOutChain, const FVector& TargetPosition, float Precision, int32 MaxIteration, bool bStartFromTail, bool bEnableRotationLimit, const TArray<float>& RotationLimitPerJoints)
	{
		struct Local
		{
			static bool UpdateChainLink(TArray<FCCDIKChainLink>& Chain, int32 LinkIndex, const FVector& TargetPos, bool bInEnableRotationLimit, const TArray<float>& InRotationLimitPerJoints)
			{
				int32 const TipBoneLinkIndex = Chain.Num() - 1;

				ensure(Chain.IsValidIndex(TipBoneLinkIndex));
				FCCDIKChainLink& CurrentLink = Chain[LinkIndex];

				// update new tip pos
				FVector TipPos = Chain[TipBoneLinkIndex].Transform.GetLocation();

				FTransform& CurrentLinkTransform = CurrentLink.Transform;
				FVector ToEnd = TipPos - CurrentLinkTransform.GetLocation();
				FVector ToTarget = TargetPos - CurrentLinkTransform.GetLocation();

				ToEnd.Normalize();
				ToTarget.Normalize();

				double RotationLimitPerJointInRadian = FMath::DegreesToRadians(InRotationLimitPerJoints[LinkIndex]);
				double Angle = FMath::ClampAngle(FMath::Acos(FVector::DotProduct(ToEnd, ToTarget)), -RotationLimitPerJointInRadian, RotationLimitPerJointInRadian);
				bool bCanRotate = (FMath::Abs(Angle) > DOUBLE_KINDA_SMALL_NUMBER) && (!bInEnableRotationLimit || RotationLimitPerJointInRadian > CurrentLink.CurrentAngleDelta);
				if (bCanRotate)
				{
					// check rotation limit first, if fails, just abort
					if (bInEnableRotationLimit)
					{
						if (RotationLimitPerJointInRadian < CurrentLink.CurrentAngleDelta + Angle)
						{
							Angle = RotationLimitPerJointInRadian - CurrentLink.CurrentAngleDelta;
							if (Angle <= DOUBLE_KINDA_SMALL_NUMBER)
							{
								return false;
							}
						}

						CurrentLink.CurrentAngleDelta += Angle;
					}

					// continue with rotating toward to target
					FVector RotationAxis = FVector::CrossProduct(ToEnd, ToTarget);
					if (RotationAxis.SizeSquared() > 0.f)
					{
						RotationAxis.Normalize();
						// Delta Rotation is the rotation to target
						FQuat DeltaRotation(RotationAxis, Angle);

						FQuat NewRotation = DeltaRotation * CurrentLinkTransform.GetRotation();
						NewRotation.Normalize();
						CurrentLinkTransform.SetRotation(NewRotation);

						// if I have parent, make sure to refresh local transform since my current transform has changed
						if (LinkIndex > 0)
						{
							FCCDIKChainLink const & Parent = Chain[LinkIndex - 1];
							CurrentLink.LocalTransform = CurrentLinkTransform.GetRelativeTransform(Parent.Transform);
							CurrentLink.LocalTransform.NormalizeRotation();
						}

						// now update all my children to have proper transform
						FTransform CurrentParentTransform = CurrentLinkTransform;

						// now update all chain
						for (int32 ChildLinkIndex = LinkIndex + 1; ChildLinkIndex <= TipBoneLinkIndex; ++ChildLinkIndex)
						{
							FCCDIKChainLink& ChildIterLink = Chain[ChildLinkIndex];
							const FTransform LocalTransform = ChildIterLink.LocalTransform;
							ChildIterLink.Transform = LocalTransform * CurrentParentTransform;
							ChildIterLink.Transform.NormalizeRotation();
							CurrentParentTransform = ChildIterLink.Transform;
						}

						return true;
					}
				}

				return false;
			}
		};

		bool bBoneLocationUpdated = false;
		int32 const NumChainLinks = InOutChain.Num();

		// iterate
		{
			int32 const TipBoneLinkIndex = NumChainLinks - 1;

			// @todo optimize locally if no update, stop?
			bool bLocalUpdated = false;
			// check how far
			const FVector TargetPos = TargetPosition;
			FVector TipPos = InOutChain[TipBoneLinkIndex].Transform.GetLocation();
			double Distance = FVector::Dist(TargetPos, TipPos);
			int32 IterationCount = 0;
			while ((Distance > Precision) && (IterationCount++ < MaxIteration))
			{
				// iterate from tip to root
				if (bStartFromTail)
				{
					for (int32 LinkIndex = TipBoneLinkIndex - 1; LinkIndex > 0; --LinkIndex)
					{
						bLocalUpdated |= Local::UpdateChainLink(InOutChain, LinkIndex, TargetPos, bEnableRotationLimit, RotationLimitPerJoints);
					}
				}
				else
				{
					for (int32 LinkIndex = 1; LinkIndex < TipBoneLinkIndex; ++LinkIndex)
					{
						bLocalUpdated |= Local::UpdateChainLink(InOutChain, LinkIndex, TargetPos, bEnableRotationLimit, RotationLimitPerJoints);
					}
				}

				Distance = FVector::Dist(InOutChain[TipBoneLinkIndex].Transform.GetLocation(), TargetPosition);

				bBoneLocationUpdated |= bLocalUpdated;

				// no more update in this iteration
				if (!bLocalUpdated)
				{
					break;
				}
			}
		}

		return bBoneLocationUpdated;
	}

}
