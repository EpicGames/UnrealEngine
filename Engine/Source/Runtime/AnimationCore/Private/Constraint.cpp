// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Constraint.cpp: Constraint implementation
=============================================================================*/

#include "Constraint.h"
#include "AnimationCoreLibrary.h"
#include "AnimationCoreUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Constraint)

void FConstraintOffset::ApplyInverseOffset(const FTransform& InTarget, FTransform& OutSource) const
{
	// in this matter, parent is accumulated first, and then individual component gets applied
	// I think that will be more consistent than going the other way
	// this parent is confusing, rename?
	OutSource = Parent.GetRelativeTransformReverse(InTarget);

	if (Translation != FVector::ZeroVector)
	{
		OutSource.AddToTranslation(Translation);
	}

	if (Rotation != FQuat::Identity)
	{
		OutSource.SetRotation(OutSource.GetRotation() * Rotation);
	}

	// I know I'm doing just != , not nearly
	if (Scale != FVector::OneVector)
	{
		OutSource.SetScale3D(OutSource.GetScale3D() * Scale);
	}
}

void FConstraintOffset::SaveInverseOffset(const FTransform& Source, const FTransform& Target, const FConstraintDescription& Operator)
{
	Reset();

	// override previous value, this is rule
	if (Operator.bParent)
	{
		Parent = Target.GetRelativeTransform(Source);
	}
	else
	{
		if (Operator.bTranslation)
		{
			Translation = Source.GetTranslation() - Target.GetTranslation();
		}

		if (Operator.bRotation)
		{
			Rotation = Source.GetRotation() * Target.GetRotation().Inverse();
		}

		if (Operator.bScale)
		{
			FVector RecipTarget = FTransform::GetSafeScaleReciprocal(Target.GetScale3D());
			Scale = Source.GetScale3D() * RecipTarget;
		}
	}
}

///////////////////////////////////////////////////////////////////
// new constraint change

void FConstraintData::ApplyInverseOffset(const FTransform& InTarget, FTransform& OutSource, const FTransform& InBaseTransform) const
{
	if (bMaintainOffset)
	{
		//The offset is saved based on 
		// (Source - Target) - BaseTransform  (SaveInverseOffset)
		// note that all of them is in component space
		// and also depending on rotation or translation or scale, how the inverse is calculated is different
		// This will get applied to 
		// Offset + [NewBaseTransform] + [NewTargetTransform] = [New SourceTransform] (ApplyInverseOffset)
		if (Constraint.DoesAffectTransform())
		{
			OutSource = (Offset * InBaseTransform) * InTarget;
		}
		else
		{
			if (Constraint.DoesAffectTranslation())
			{
				OutSource.SetTranslation(InTarget.GetTranslation() + InBaseTransform.TransformVectorNoScale(Offset.GetTranslation()));
			}

			if (Constraint.DoesAffectRotation())
			{
				OutSource.SetRotation(InTarget.GetRotation() * InBaseTransform.GetRotation() * Offset.GetRotation());
				OutSource.NormalizeRotation();
			}

			if (Constraint.DoesAffectScale())
			{
				OutSource.SetScale3D(InTarget.GetScale3D() * InBaseTransform.GetScale3D() * Offset.GetScale3D());
			}
		}
	}
	else
	{
		OutSource = InTarget;
	}
}

void FConstraintData::SaveInverseOffset(const FTransform& Source, const FTransform& Target, const FTransform& InBaseTransform)
{
	ResetOffset();

	if (bMaintainOffset)
	{
		//The offset is saved based on 
		// (Source - Target) - BaseTransform  (SaveInverseOffset)
		// note that all of them is in component space
		// and also depending on rotation or translation or scale, how the inverse is calculated is different
		// This will get applied to 
		// Offset + [NewBaseTransform] + [NewTargetTransform] = [New SourceTransform] (ApplyInverseOffset)
		if (Constraint.DoesAffectTransform())
		{
			FTransform ToSource = Source.GetRelativeTransform(Target);
			Offset = ToSource.GetRelativeTransform(InBaseTransform);
		}
		else
		{
			if (Constraint.DoesAffectTranslation())
			{
				FVector DeltaLocation = Source.GetTranslation() - Target.GetTranslation();
				Offset.SetLocation(InBaseTransform.InverseTransformVectorNoScale(DeltaLocation));
			}

			if (Constraint.DoesAffectRotation())
			{
				// this is same as local target's inverse * local source
				// (target.Inverse() * base) * (source.Inverse() * base).inverse()
				// = (target.Inverse() * base * base.Inverse() * source
				// = target.Inverse() * source
				FQuat DeltaRotation = Target.GetRotation().Inverse() * Source.GetRotation();
				Offset.SetRotation(InBaseTransform.GetRotation().Inverse() * DeltaRotation);
				Offset.NormalizeRotation();
			}

			if (Constraint.DoesAffectScale())
			{
				FVector RecipTarget = FTransform::GetSafeScaleReciprocal(Target.GetScale3D());
				FVector DeltaScale = Source.GetScale3D() * RecipTarget;
				FVector RecipBase = FTransform::GetSafeScaleReciprocal(InBaseTransform.GetScale3D());
				Offset.SetScale3D(DeltaScale * RecipBase);
			}
		}
	}
}

void FConstraintData::ApplyConstraintTransform(const FTransform& TargetTransform, const FTransform& InCurrentTransform, const FTransform& CurrentParentTransform, FMultiTransformBlendHelper& BlendHelperInLocalSpace) const
{
	FTransform OffsetTargetTransform;

	// now apply inverse on the target since that's what we're applying
	ApplyInverseOffset(TargetTransform, OffsetTargetTransform, CurrentParentTransform);

	// give the offset target transform
	Constraint.ApplyConstraintTransform(OffsetTargetTransform, InCurrentTransform, CurrentParentTransform, Weight, BlendHelperInLocalSpace);
}

void FTransformConstraintDescription::AccumulateConstraintTransform(const FTransform& TargetTransform, const FTransform& CurrentTransform, const FTransform& CurrentParentTransform, float Weight, FMultiTransformBlendHelper& BlendHelperInLocalSpace) const
{
	FTransform TargetLocalTransform = TargetTransform.GetRelativeTransform(CurrentParentTransform);

	if (DoesAffectTransform())
	{
		BlendHelperInLocalSpace.AddParent(TargetLocalTransform, Weight);
	}
	else
	{
		if (DoesAffectTranslation())
		{
			FVector Translation = TargetLocalTransform.GetTranslation();
			AxesFilterOption.FilterVector(Translation);
			BlendHelperInLocalSpace.AddTranslation(Translation, Weight);
		}

		if (DoesAffectRotation())
		{
			FQuat DeltaRotation = TargetLocalTransform.GetRotation();
			AxesFilterOption.FilterQuat(DeltaRotation);
			BlendHelperInLocalSpace.AddRotation(DeltaRotation, Weight);
		}

		if (DoesAffectScale())
		{
			FVector Scale = TargetLocalTransform.GetScale3D();
			AxesFilterOption.FilterVector(Scale);
			BlendHelperInLocalSpace.AddScale(Scale, Weight);
		}
	}
}

void FAimConstraintDescription::AccumulateConstraintTransform(const FTransform& TargetTransform, const FTransform& CurrentTransform, const FTransform& CurrentParentTransform, float Weight, FMultiTransformBlendHelper& BlendHelperInLocalSpace) const
{
	// need current transform - I need global transform of Target, I think incoming is local space
	FTransform NewTransform = CurrentTransform;

	if (bUseLookUp)
	{
		FQuat DeltaRotation = AnimationCore::SolveAim(NewTransform, LookUpTarget, LookUp_Axis.GetTransformedAxis(NewTransform), false, FVector::ZeroVector);
		NewTransform.SetRotation(DeltaRotation * NewTransform.GetRotation());
	}

	FQuat DeltaRotation = AnimationCore::SolveAim(NewTransform, TargetTransform.GetLocation(), LookAt_Axis.GetTransformedAxis(NewTransform), false, FVector::ZeroVector);
	NewTransform.SetRotation(DeltaRotation * NewTransform.GetRotation());

	FTransform LocalTransform = NewTransform.GetRelativeTransform(CurrentParentTransform);
	FQuat LocalRotation = LocalTransform.GetRotation();
	AxesFilterOption.FilterQuat(LocalRotation);
	BlendHelperInLocalSpace.AddRotation(LocalRotation, Weight);
}
