// Copyright Epic Games, Inc. All Rights Reserved.

#include "EulerTransform.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EulerTransform)

// Identity transform
const FEulerTransform FEulerTransform::Identity(FVector(0.0f), FRotator(0.0f, 0.0f, 0.0f), FVector(1.0f));
