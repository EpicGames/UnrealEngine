// Copyright Epic Games, Inc. All Rights Reserved.

#include "FABRIK.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FABRIK)

namespace AnimationCore
{
	/////////////////////////////////////////////////////
	// Implementation of the FABRIK IK Algorithm
	// Please see http://andreasaristidou.com/publications/FABRIK.pdf for more details

	bool SolveFabrik(TArray<FFABRIKChainLink>& InOutChain, const FVector& TargetPosition, double MaximumReach, double Precision, int32 MaxIterations)
	{
		bool bBoneLocationUpdated = false;
		double const RootToTargetDistSq = FVector::DistSquared(InOutChain[0].Position, TargetPosition);
		int32 const NumChainLinks = InOutChain.Num();

		// FABRIK algorithm - bone translation calculation
		// If the effector is further away than the distance from root to tip, simply move all bones in a line from root to effector location
		if (RootToTargetDistSq > FMath::Square(MaximumReach))
		{
			for (int32 LinkIndex = 1; LinkIndex < NumChainLinks; LinkIndex++)
			{
				FFABRIKChainLink const & ParentLink = InOutChain[LinkIndex - 1];
				FFABRIKChainLink & CurrentLink = InOutChain[LinkIndex];
				CurrentLink.Position = ParentLink.Position + (TargetPosition - ParentLink.Position).GetUnsafeNormal() * CurrentLink.Length;
			}
			bBoneLocationUpdated = true;
		}
		else // Effector is within reach, calculate bone translations to position tip at effector location
		{
			int32 const TipBoneLinkIndex = NumChainLinks - 1;

			// Check distance between tip location and effector location
			double Slop = FVector::Dist(InOutChain[TipBoneLinkIndex].Position, TargetPosition);
			if (Slop > Precision)
			{
				// Set tip bone at end effector location.
				InOutChain[TipBoneLinkIndex].Position = TargetPosition;

				int32 IterationCount = 0;
				while ((Slop > Precision) && (IterationCount++ < MaxIterations))
				{
					// "Forward Reaching" stage - adjust bones from end effector.
					for (int32 LinkIndex = TipBoneLinkIndex - 1; LinkIndex > 0; LinkIndex--)
					{
						FFABRIKChainLink & CurrentLink = InOutChain[LinkIndex];
						FFABRIKChainLink const & ChildLink = InOutChain[LinkIndex + 1];

						CurrentLink.Position = ChildLink.Position + (CurrentLink.Position - ChildLink.Position).GetUnsafeNormal() * ChildLink.Length;
					}

					// "Backward Reaching" stage - adjust bones from root.
					for (int32 LinkIndex = 1; LinkIndex < TipBoneLinkIndex; LinkIndex++)
					{
						FFABRIKChainLink const & ParentLink = InOutChain[LinkIndex - 1];
						FFABRIKChainLink & CurrentLink = InOutChain[LinkIndex];

						CurrentLink.Position = ParentLink.Position + (CurrentLink.Position - ParentLink.Position).GetUnsafeNormal() * CurrentLink.Length;
					}

					// Re-check distance between tip location and effector location
					// Since we're keeping tip on top of effector location, check with its parent bone.
					Slop = FMath::Abs(InOutChain[TipBoneLinkIndex].Length - FVector::Dist(InOutChain[TipBoneLinkIndex - 1].Position, TargetPosition));
				}

				// Place tip bone based on how close we got to target.
				{
					FFABRIKChainLink const & ParentLink = InOutChain[TipBoneLinkIndex - 1];
					FFABRIKChainLink & CurrentLink = InOutChain[TipBoneLinkIndex];

					CurrentLink.Position = ParentLink.Position + (CurrentLink.Position - ParentLink.Position).GetUnsafeNormal() * CurrentLink.Length;
				}

				bBoneLocationUpdated = true;
			}
		}

		return bBoneLocationUpdated;
	}
}

