// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoftIK.h"

namespace AnimationCore
{
	void SoftenIKEffectorPosition(
		const FVector& RootLocation,
		const float TotalChainLength,
		const float SoftLengthPercent,
		const float Alpha,
		FVector& InOutEffectorPosition)
	{
		if (FMath::Abs(1.0f - SoftLengthPercent) < KINDA_SMALL_NUMBER)
		{
			// soft length near zero
			return;
		}

		if (TotalChainLength <= KINDA_SMALL_NUMBER)
		{
			// chain length near zero
			return;
		}
		
		// get vector from root of chain to effector (pre adjusted)
		FVector StartToEffector = InOutEffectorPosition - RootLocation;
		float CurrentLength;
		StartToEffector.ToDirectionAndLength(StartToEffector, CurrentLength);

		// convert percentage to distance
		const float SoftDistance = TotalChainLength * (1.0f - FMath::Min(1.0f, SoftLengthPercent));
		const float HardLength = TotalChainLength - SoftDistance;
		const float CurrentDelta = CurrentLength - HardLength;
		if (CurrentDelta <= KINDA_SMALL_NUMBER || SoftDistance <= KINDA_SMALL_NUMBER)
		{
			// not in the soft zone
			return;
		}

		// calculate the "softened" length of the effector
		const float PercentIntoSoftLength = CurrentDelta / SoftDistance;
		const float SoftenedLength = HardLength + SoftDistance * (1.0 - FMath::Exp(-PercentIntoSoftLength));

		// apply the new effector location
		float FinalAlpha = FMath::Clamp(Alpha, 0.0, 1.0f);
		if (FinalAlpha < 1.0f)
		{
			// alpha blend the softness (optional)
			const float MaxLength = FMath::Min(CurrentLength, TotalChainLength);
			const float AlphaBlendedLength = FMath::Lerp(MaxLength, SoftenedLength, Alpha);
			InOutEffectorPosition = RootLocation + StartToEffector * AlphaBlendedLength;
		}
		else
		{
			// use the soft position directly
			InOutEffectorPosition = RootLocation + StartToEffector * SoftenedLength;
		}
	}
}
