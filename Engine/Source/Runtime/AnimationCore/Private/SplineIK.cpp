// Copyright Epic Games, Inc. All Rights Reserved.

#include "SplineIK.h"
#include "HAL/ThreadSingleton.h"

namespace AnimationCore
{
	struct FSplineIKSolverScratchArea : public TThreadSingleton<FSplineIKSolverScratchArea>
	{
		TArray<float> SplineAlphas;
	};

	void SolveSplineIK(const TArray<FTransform>& BoneTransforms, const FInterpCurveVector& PositionSpline, const FInterpCurveQuat& RotationSpline, const FInterpCurveVector& ScaleSpline, const float TotalSplineAlpha, const float TotalSplineLength, const FFloatMapping& Twist, const float Roll, const float Stretch, const float Offset, const EAxis::Type BoneAxis, const FFindParamAtFirstSphereIntersection& FindParamAtFirstSphereIntersection, const TArray<FQuat>& BoneOffsetRotations, const TArray<float>& BoneLengths, const float OriginalSplineLength, TArray<FTransform>& OutBoneTransforms)
	{
		check(BoneTransforms.Num() == BoneLengths.Num());
		check(BoneTransforms.Num() == BoneOffsetRotations.Num());
		check(Twist.IsBound());
		check(FindParamAtFirstSphereIntersection.IsBound());

		OutBoneTransforms.Reset();

		const float TotalStretchRatio = FMath::Lerp(OriginalSplineLength, TotalSplineLength, Stretch) / OriginalSplineLength;

		FVector PreviousPoint;
		int32 StartingLinearIndex = 0;
		float InitialAlpha = 0.0f;
		if (Offset == 0.0f)
		{
			PreviousPoint = PositionSpline.Points[0].OutVal;
		}
		else
		{
			InitialAlpha = FindParamAtFirstSphereIntersection.Execute(PositionSpline.Points[0].OutVal, Offset, StartingLinearIndex);
			PreviousPoint = PositionSpline.Eval(InitialAlpha);
		}

		const int32 BoneCount = BoneTransforms.Num();

		TArray<float>& SplineAlphas = FSplineIKSolverScratchArea::Get().SplineAlphas;
		SplineAlphas.SetNum(BoneCount);
		OutBoneTransforms.SetNum(BoneCount);

		// First calculate positions & scales
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; BoneIndex++)
		{
			SplineAlphas[BoneIndex] = BoneIndex > 0 ? FindParamAtFirstSphereIntersection.Execute(PreviousPoint, BoneLengths[BoneIndex] * TotalStretchRatio, StartingLinearIndex) : InitialAlpha;

			FTransform& BoneTransform = OutBoneTransforms[BoneIndex];
			BoneTransform.SetLocation(PositionSpline.Eval(SplineAlphas[BoneIndex]));
			BoneTransform.SetScale3D(ScaleSpline.Eval(SplineAlphas[BoneIndex]));

			PreviousPoint = BoneTransform.GetLocation();
		}

		// Now calculate rotations
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; BoneIndex++)
		{
			FTransform& BoneTransform = OutBoneTransforms[BoneIndex];

			// Get the rotation that the spline provides
			FQuat SplineRotation = RotationSpline.Eval(SplineAlphas[BoneIndex]);

			// Build roll/twist rotation
			const float TotalRoll = Roll + Twist.Execute(SplineAlphas[BoneIndex] / TotalSplineAlpha);
			FQuat RollRotation = FRotator(BoneAxis == EAxis::Y ? TotalRoll : 0.0f, BoneAxis == EAxis::X ? TotalRoll : 0.0f, BoneAxis == EAxis::Z ? TotalRoll : 0.0f).Quaternion();

			// Correct rotation of bone to align our orientation onto the spline
			FQuat DirectionCorrectingRotation(FQuat::Identity);
			FQuat BoneOffsetRotation(FQuat::Identity);
			if (BoneIndex < BoneCount - 1)
			{
				FVector NewBoneDir = OutBoneTransforms[BoneIndex + 1].GetLocation() - BoneTransform.GetLocation();

				// Only try and correct direction if we get a non-zero tangent.
				if (NewBoneDir.Normalize())
				{
					// Calculate the direction that bone is currently pointing.
					const FVector CurrentBoneDir = BoneTransforms[BoneIndex + 1].GetUnitAxis(BoneAxis).GetSafeNormal();

					// Calculate a quaternion that gets us from our current rotation to the desired one.
					DirectionCorrectingRotation = FQuat::FindBetweenNormals(CurrentBoneDir, NewBoneDir);
				}

				BoneOffsetRotation = BoneOffsetRotations[BoneIndex + 1];
			}

			BoneTransform.SetRotation(RollRotation * DirectionCorrectingRotation * BoneOffsetRotation * SplineRotation);
		}
	}
}
