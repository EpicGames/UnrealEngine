// Copyright Epic Games, Inc. All Rights Reserved.

#include "BoneWeights.h"
#include "Misc/AutomationTest.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/BufferReader.h"

#include <initializer_list>

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBoneWeightTestBasic, "System.AnimationCore.BoneWeight.Basic", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FBoneWeightTestBasic::RunTest(const FString& Parameters)
{
	using namespace UE::AnimationCore;

	// Correct values (uint8)
	{
		FBoneWeight BWMin(FBoneIndexType(2), uint8(0));
		UTEST_EQUAL(TEXT("FBoneWeight index (uint8)"), BWMin.GetBoneIndex(), 2);
		UTEST_EQUAL(TEXT("FBoneWeight min value (uint8)"), BWMin.GetWeight(), 0.0f);
		UTEST_EQUAL(TEXT("FBoneWeight min raw value (uint8)"), BWMin.GetRawWeight(), uint16(0));

		// Because we construct a uin16 value from uint8 by doubling (v | v << 8). The reason
		// is so that we end up with a fully saturated value (0xFFFF) for uint8(0xFF), rather 
		// than (0xFF00) which would not be a full weight.
		FBoneWeight BWMid(FBoneIndexType(0), uint8(127));
		UTEST_EQUAL(TEXT("FBoneWeight mid value (uint8)"), BWMid.GetWeight(), 0x7F7F / 65535.0f);
		UTEST_EQUAL(TEXT("FBoneWeight mid raw value (uint8)"), BWMid.GetRawWeight(), 0x7F7FU);

		FBoneWeight BWMax(FBoneIndexType(0), uint8(255));
		UTEST_EQUAL(TEXT("FBoneWeight max value (uint8)"), BWMax.GetWeight(), 1.0f);
		UTEST_EQUAL(TEXT("FBoneWeight max raw value (uint8)"), BWMax.GetRawWeight(), FBoneWeight::GetMaxRawWeight());
	}

	// Correct values (uint16)
	{
		FBoneWeight BWMin(FBoneIndexType(2), uint16(0));
		UTEST_EQUAL(TEXT("FBoneWeight index (uint8)"), BWMin.GetBoneIndex(), 2);
		UTEST_EQUAL(TEXT("FBoneWeight min value (uint16)"), BWMin.GetWeight(), 0.0f);
		UTEST_EQUAL(TEXT("FBoneWeight min raw value (uint16)"), BWMin.GetRawWeight(), uint16(0));

		// Since uint16::max / 2 -is equal to 32767.5
		FBoneWeight BWMid(FBoneIndexType(0), uint16(0x7FFF));
		UTEST_EQUAL(TEXT("FBoneWeight mid value (uint16)"), BWMid.GetWeight(), 0x7FFF / float(FBoneWeight::GetMaxRawWeight()));
		UTEST_EQUAL(TEXT("FBoneWeight mid raw value (uint16)"), BWMid.GetRawWeight(), 0x7FFF);

		FBoneWeight BWMax(FBoneIndexType(0), uint16(FBoneWeight::GetMaxRawWeight()));
		UTEST_EQUAL(TEXT("FBoneWeight max value (uint16)"), BWMax.GetWeight(), 1.0f);
		UTEST_EQUAL(TEXT("FBoneWeight max raw value (uint16)"), BWMax.GetRawWeight(), FBoneWeight::GetMaxRawWeight());
	}

	// Copy constructor
	{
		FBoneWeight BWOrig(FBoneIndexType(2), uint16(0x8000));
		FBoneWeight BWCopy(BWOrig);
		UTEST_EQUAL(TEXT("FBoneWeight copied index"), BWCopy.GetBoneIndex(), 2);
		UTEST_EQUAL(TEXT("FBoneWeight copied value"), BWCopy.GetRawWeight(), 0x8000);
	}

	// Archiving
	{
		FBufferArchive BA;
		FBoneWeight BWOrig(FBoneIndexType(2), uint16(0x8000));
		BA << BWOrig;

		FBoneWeight BWRead;
		FBufferReader BR(BA.GetData(), BA.Num(), /*bFreeOnClose=*/false);
		BR << BWRead;
		UTEST_EQUAL(TEXT("FBoneWeight::Archive[Index]"), BWRead.GetBoneIndex(), 2);
		UTEST_EQUAL(TEXT("FBoneWeight::Archive[Weight]"), BWRead.GetRawWeight(), 0x8000);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBoneWeightsSettingsTest, "System.AnimationCore.BoneWeights.Settings", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FBoneWeightsSettingsTest::RunTest(const FString& Parameters)
{
	using namespace UE::AnimationCore;

	// Ensure that no-one's tampered with the defaults.
	FBoneWeightsSettings S;

	UTEST_EQUAL(TEXT("FBoneWeightsSettings NormalizeType"), S.GetNormalizeType(), EBoneWeightNormalizeType::Always);
	UTEST_EQUAL(TEXT("FBoneWeightsSettings MaxWeightCount"), S.GetMaxWeightCount(), MaxInlineBoneWeightCount);
	UTEST_EQUAL(TEXT("FBoneWeightsSettings Weight Threshold"), S.GetRawWeightThreshold(), 1);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBoneWeightsTestBasic, "System.AnimationCore.BoneWeights.Basic", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FBoneWeightsTestBasic::RunTest(const FString& Parameters)
{
	using namespace UE::AnimationCore;

	{
		const TCHAR* Name = TEXT("FBoneWeights::FBoneWeights");

		FBoneWeights	A;
		UTEST_EQUAL(Name, A.Num(), 0);

		FBoneWeights	B(A);
		UTEST_EQUAL(Name, B.Num(), 0);
	}

	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight");

		FBoneWeights	A;
		A.SetBoneWeight(FBoneIndexType(0), 1.0f);
		UTEST_EQUAL(Name, A.Num(), 1);

		FBoneWeights	B(A);
		UTEST_EQUAL(Name, B.Num(), 1);
	}

	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight");

		FBoneWeights	A;
		A.SetBoneWeight(FBoneIndexType(0), 1.0f);

		UTEST_EQUAL(Name, A.Num(), 1);

		FBoneWeight BI = A[0];
		UTEST_EQUAL(Name, BI.GetWeight(), 1.0f);
		UTEST_EQUAL(Name, BI.GetRawWeight(), FBoneWeight::GetMaxRawWeight());
	}

	// Check normalization works
	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight[MaxWNormalize=AboveOne]");

		FBoneWeightsSettings S;
		S.SetNormalizeType(EBoneWeightNormalizeType::AboveOne);

		FBoneWeights	A;
		A.SetBoneWeight(FBoneIndexType(0), 1.0f, S);
		A.SetBoneWeight(FBoneIndexType(1), 1.0f, S);

		UTEST_EQUAL(Name, A.Num(), 2);
		UTEST_EQUAL(Name, A[0].GetWeight(), 0.5f);
	}
	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight[Normalize=Always]");

		FBoneWeightsSettings S;
		S.SetNormalizeType(EBoneWeightNormalizeType::Always);

		FBoneWeights	A;
		// Gets normalized to 1.0
		A.SetBoneWeight(FBoneIndexType(0), 0.25f, S);

		// Gets normalized to 0.25 / 1.25 = 0.2 leaving the first entry at 0.8
		A.SetBoneWeight(FBoneIndexType(1), 0.25f, S);

		UTEST_EQUAL(Name, A.Num(), 2);
		UTEST_EQUAL(Name, A[0].GetWeight(), 0.8f);
	}
	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight[Normalize=None]");

		FBoneWeightsSettings S;
		S.SetNormalizeType(EBoneWeightNormalizeType::None);

		FBoneWeights	A;
		A.SetBoneWeight(FBoneIndexType(0), 0.25f, S);
		A.SetBoneWeight(FBoneIndexType(1), 0.25f, S);

		UTEST_EQUAL(Name, A.Num(), 2);
		UTEST_EQUAL(Name, A[0].GetWeight(), 0.25f);
	}

	// Check overwrite same bone index works  (with renormalization)
	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight[Normalize=AboveOne, Overwrite]");

		FBoneWeightsSettings S;
		S.SetNormalizeType(EBoneWeightNormalizeType::AboveOne);

		FBoneWeights	A;
		A.SetBoneWeight(FBoneIndexType(0), 0.25f, S);
		A.SetBoneWeight(FBoneIndexType(1), 0.25f, S);

		UTEST_EQUAL(Name, A.Num(), 2);
		UTEST_EQUAL(Name, A[0].GetWeight(), 0.25f);

		// This should set the total weight to above 1.0 and hence trigger normalization.
		A.SetBoneWeight(FBoneIndexType(1), 1.0f, S);
		UTEST_EQUAL(Name, A[0].GetWeight(), 0.8f);
	}

	// Check normalization correctly distributes error.
	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight[Normalize=Always, Distribute]");

		// Ensure that no weights get thrown out because they're too small.
		FBoneWeightsSettings SN, SA;
		SN.SetNormalizeType(EBoneWeightNormalizeType::None);
		SN.SetWeightThreshold(0.0f);
		SA.SetNormalizeType(EBoneWeightNormalizeType::Always);
		SA.SetWeightThreshold(0.0f);

		FBoneWeights A;
		for (uint32 Index = 0; Index < MaxInlineBoneWeightCount; Index++)
		{
			A.SetBoneWeight(FBoneIndexType(Index), Index == 0 ? 1.0f : 2.0f / float(FBoneWeight::GetMaxRawWeight()), SN);
		}

		A.Renormalize(SA);

		UTEST_EQUAL(Name, A.Num(), MaxInlineBoneWeightCount);
		
		// None of the small weights should not have changed in value, since the total scaling 
		// (1.0 - 22/65535) is so close to 1.0 that it won't make a difference with 
		// round-to-nearest.
		int32 WeightSum = 0;
		for (int32 Index = 0; Index < A.Num(); Index++)
		{
			WeightSum += int32(A[Index].GetRawWeight());
			if (Index > 0)
			{
				FString NameSub = FString::Printf(TEXT("%s[%d]"), Name, Index);
				UTEST_EQUAL(NameSub, int32(A[Index].GetRawWeight()), 2);
			}
		}
		UTEST_EQUAL(Name, WeightSum, int32(FBoneWeight::GetMaxRawWeight()));


	}

	// Check ordering works
	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight[Ordering]");

		FBoneWeightsSettings S;
		S.SetNormalizeType(EBoneWeightNormalizeType::None);

		FBoneWeights	A;
		A.SetBoneWeight(FBoneIndexType(0), 0.25f, S);
		A.SetBoneWeight(FBoneIndexType(1), 0.5f, S);

		UTEST_EQUAL(Name, A.Num(), 2);
		UTEST_EQUAL(Name, A[0].GetBoneIndex(), 1);
	}

	// Check max limit works
	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight[MaxWeights]");

		FBoneWeightsSettings S;
		S.SetMaxWeightCount(3);
		S.SetNormalizeType(EBoneWeightNormalizeType::None);

		FBoneWeights	A;
		A.SetBoneWeight(FBoneIndexType(0), 0.25f, S);
		A.SetBoneWeight(FBoneIndexType(1), 0.20f, S);
		A.SetBoneWeight(FBoneIndexType(2), 0.15f, S);
		A.SetBoneWeight(FBoneIndexType(3), 0.10f, S);
		A.SetBoneWeight(FBoneIndexType(4), 0.05f, S);

		UTEST_EQUAL(Name, A.Num(), 3);
		UTEST_EQUAL(Name, A[2].GetBoneIndex(), 2);

		// Ensure it gets inserted just before the 2/0.15f value.
		A.SetBoneWeight(FBoneIndexType(5), 0.17f, S);
		UTEST_EQUAL(Name, A.Num(), 3);
		UTEST_EQUAL(Name, A[2].GetBoneIndex(), 5);
	}

	// Check threshold works
	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight[Threshold, Normalize=None]");
		FBoneWeightsSettings S;
		S.SetWeightThreshold(0.40f);
		S.SetNormalizeType(EBoneWeightNormalizeType::None);

		FBoneWeights	A;
		A.SetBoneWeight(FBoneIndexType(0), 1.00f, S);
		A.SetBoneWeight(FBoneIndexType(1), 0.75f, S);
		A.SetBoneWeight(FBoneIndexType(2), 0.50f, S);

		UTEST_EQUAL(Name, A.Num(), 3);
		UTEST_EQUAL(Name, A[2].GetBoneIndex(), 2);

		A.SetBoneWeight(FBoneIndexType(3), 0.25f, S);
		UTEST_EQUAL(Name, A.Num(), 3);
		UTEST_EQUAL(Name, A[2].GetBoneIndex(), 2);

		A.SetBoneWeight(FBoneIndexType(4), 0.45f, S);
		UTEST_EQUAL(Name, A.Num(), 4);
		UTEST_EQUAL(Name, A[3].GetBoneIndex(), 4);

		// Set an existing bone index to be below threshold. It should get removed.
		A.SetBoneWeight(FBoneIndexType(1), 0.1f, S);
		UTEST_EQUAL(Name, A.Num(), 3);
		UTEST_EQUAL(Name, A[2].GetBoneIndex(), 4);
	}

	{
		const TCHAR* Name = TEXT("FBoneWeights::SetBoneWeight[Threshold, Normalize=Always]");
		FBoneWeightsSettings S;
		S.SetWeightThreshold(0.25f);
		S.SetNormalizeType(EBoneWeightNormalizeType::Always);

		FBoneWeights	A;
		A.SetBoneWeight(FBoneIndexType(0), 1.00f, S);
		// Weights = [1.0]
		UTEST_EQUAL(Name, A.Num(), 1);
		UTEST_EQUAL(Name, A[0].GetBoneIndex(), 0);


		// Add one more, this should pass the threshold, both before and after normalization.
		A.SetBoneWeight(FBoneIndexType(1), 0.5f, S);
		// Weights = [0.666, 0.333]
		UTEST_EQUAL(Name, A.Num(), 2);
		UTEST_EQUAL(Name, A[1].GetBoneIndex(), 1);


		// Add one more, this passes the early threshold check, but not after normalization.
		A.SetBoneWeight(FBoneIndexType(2), 0.25f, S);
		// Weights = [0.5328, 0.2664, 0.2] -> Last entry culled.
		UTEST_EQUAL(Name, A.Num(), 2);
		UTEST_EQUAL(Name, A[1].GetBoneIndex(), 1);
	}

	// Check removal works
	{
		const TCHAR* Name = TEXT("FBoneWeights::RemoveBoneWeight");

		// Don't normalize at first.
		FBoneWeights A;
		A.SetBoneWeight(FBoneIndexType(0), 0.5f);
		A.SetBoneWeight(FBoneIndexType(1), 0.5f);

		UTEST_EQUAL(Name, A.Num(), 2);

		A.RemoveBoneWeight(FBoneIndexType(2));
		UTEST_EQUAL(Name, A.Num(), 2);

		// Normalize on remove to see if the weight changes.
		FBoneWeightsSettings S;
		S.SetNormalizeType(EBoneWeightNormalizeType::Always);
		A.RemoveBoneWeight(FBoneIndexType(0), S);

		UTEST_EQUAL(Name, A.Num(), 1);
		UTEST_EQUAL(Name, A[0].GetWeight(), 1.0f);
	}

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBoneWeightsTestCreate, "System.AnimationCore.BoneWeights.Create", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FBoneWeightsTestCreate::RunTest(const FString& Parameters)
{
	using namespace UE::AnimationCore;

	{
		static const FBoneIndexType Bones[MaxInlineBoneWeightCount] = { 0, 1, 2, 3, 0, 0, 0, 0, 0, 0, 0, 0 };
		static const uint16 Influences[MaxInlineBoneWeightCount] = { 65535, 8191, 16383, 32767, 0, 0, 0, 0, 0, 0, 0, 0 };

		{
			const TCHAR* Name = TEXT("FBoneWeights::Create[FSoftSkinVertex, Threshold]");
			FBoneWeightsSettings S;
			S.SetWeightThreshold(0.20f);
			FBoneWeights BW = FBoneWeights::Create(Bones, Influences, S);

			UTEST_EQUAL(Name, BW.Num(), 3);
			UTEST_EQUAL(Name, BW[2].GetBoneIndex(), 2);
		}
		{
			const TCHAR* Name = TEXT("FBoneWeights::Create[FSoftSkinVertex, MaxWeights]");
			FBoneWeightsSettings S;
			S.SetMaxWeightCount(2);
			FBoneWeights BW = FBoneWeights::Create(Bones, Influences, S);

			UTEST_EQUAL(Name, BW.Num(), 2);
			UTEST_EQUAL(Name, BW[1].GetBoneIndex(), 3);
		}
	}

	{
		static const FBoneIndexType Bones[] = { 0, 1, 2, 3};
		static const float Influences[] = { 1.0f, 0.125f, 0.25f, 0.5f};

		static const int32 ArraySize = sizeof(Bones) / sizeof(Bones[0]);

		{
			const TCHAR* Name = TEXT("FBoneWeights::Create[Arrays, Threshold]");
			FBoneWeightsSettings S;
			S.SetWeightThreshold(0.20f);
			FBoneWeights BW = FBoneWeights::Create(Bones, Influences, ArraySize, S);

			UTEST_EQUAL(Name, BW.Num(), 3);
			UTEST_EQUAL(Name, BW[2].GetBoneIndex(), 2);
		}
		{
			const TCHAR* Name = TEXT("FBoneWeights::Create[Arrays, MaxWeights]");
			FBoneWeightsSettings S;
			S.SetMaxWeightCount(2);
			FBoneWeights BW = FBoneWeights::Create(Bones, Influences, ArraySize, S);

			UTEST_EQUAL(Name, BW.Num(), 2);
			UTEST_EQUAL(Name, BW[1].GetBoneIndex(), 3);
		}
	}
	return true;
}

struct FSimpleWeight
{
	FSimpleWeight(int Index, float Weight) : Index(Index), Weight(Weight) {}
	int Index;
	float Weight;
};

static UE::AnimationCore::FBoneWeights CreateWeights(
	std::initializer_list<FSimpleWeight> InWeights
	)
{
	using namespace UE::AnimationCore;

	// We only add raw values.
	FBoneWeightsSettings S;
	S.SetNormalizeType(EBoneWeightNormalizeType::None);

	TArray<FBoneWeight> BWS;
	BWS.Reserve(InWeights.size());
	for (const FSimpleWeight& SW : InWeights)
	{
		BWS.Add(FBoneWeight(FBoneIndexType(SW.Index), SW.Weight));
	}

	return FBoneWeights::Create(BWS, S);
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FBoneWeightsTestBlend, "System.AnimationCore.BoneWeights.Blend", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FBoneWeightsTestBlend::RunTest(const FString& Parameters)
{
	using namespace UE::AnimationCore;

	{
		const TCHAR* Name = TEXT("FBoneWeights::Blend[Both Empty]");
		auto BWA = CreateWeights({});
		auto BWB = CreateWeights({});

		auto BW = FBoneWeights::Blend(BWA, BWB, 0.5f);
		UTEST_EQUAL(Name, BW.Num(), 0);
	}
	{
		const TCHAR* Name = TEXT("FBoneWeights::Blend[One Empty]");
		auto BW0 = CreateWeights({});
		auto BW1 = CreateWeights({{0, 0.6f}});
		auto BW2 = CreateWeights({ {1, 0.5f}, {2, 0.25f} });

		auto BWA = FBoneWeights::Blend(BW0, BW1, 0.5f);
		UTEST_EQUAL(Name, BWA.Num(), 1);

		auto BWB = FBoneWeights::Blend(BW2, BW0, 0.5f);
		UTEST_EQUAL(Name, BWB.Num(), 2);
	}

		
	{
		const TCHAR* Name = TEXT("FBoneWeights::Blend[One Each, Disjoint]");

		auto BW1 = CreateWeights({ {0, 0.6f} });
		auto BW2 = CreateWeights({ {1, 0.4f} });

		auto BW = FBoneWeights::Blend(BW1, BW2, 0.5f);

		UTEST_EQUAL(Name, BW.Num(), 2);
		UTEST_EQUAL(Name, BW[0].GetBoneIndex(), 0);
		UTEST_EQUAL(Name, BW[1].GetBoneIndex(), 1);
	}

	{
		const TCHAR* Name = TEXT("FBoneWeights::Blend[One Each, Overlap]");

		auto BW1 = CreateWeights({ {0, 0.9f} });
		auto BW2 = CreateWeights({ {0, 0.1f} });

		FBoneWeightsSettings SN;
		SN.SetNormalizeType(EBoneWeightNormalizeType::None);
		auto BWN = FBoneWeights::Blend(BW1, BW2, 0.5f, SN);

		// With no normalization, the blended value will be 0.5
		UTEST_EQUAL(Name, BWN.Num(), 1);
		UTEST_EQUAL(Name, BWN[0].GetWeight(), 0.5f);

		FBoneWeightsSettings SA;
		SA.SetNormalizeType(EBoneWeightNormalizeType::Always);
		auto BWA = FBoneWeights::Blend(BW1, BW2, 0.5f, SA);

		// With full normalization, the blended value will be 1.0
		UTEST_EQUAL(Name, BWA.Num(), 1);
		UTEST_EQUAL(Name, BWA[0].GetWeight(), 1.0f);
	}

	{
		const TCHAR* Name = TEXT("FBoneWeights::Blend[Two Each, One Overlap]");

		auto BW1 = CreateWeights({ {0, 1.0f}, {1, 0.5f} });
		auto BW2 = CreateWeights({ {1, 0.5f}, {2, 0.25f} });

		FBoneWeightsSettings S;
		S.SetNormalizeType(EBoneWeightNormalizeType::None);
		S.SetBlendZeroInfluence(false);
		
		auto BW = FBoneWeights::Blend(BW1, BW2, 0.5f, S);

		UTEST_EQUAL(Name, BW.Num(), 3);
		UTEST_EQUAL(Name, BW[0].GetBoneIndex(), 0);
		UTEST_EQUAL(Name, BW[0].GetWeight(), 1.0f);
		UTEST_EQUAL(Name, BW[1].GetBoneIndex(), 1);
		UTEST_EQUAL(Name, BW[1].GetWeight(), 0.5f);
		UTEST_EQUAL(Name, BW[2].GetBoneIndex(), 2);
		UTEST_EQUAL(Name, BW[2].GetWeight(), 0.25f);

		S.SetBlendZeroInfluence(true);
		BW = FBoneWeights::Blend(BW1, BW2, 0.3f, S);

		UTEST_EQUAL(Name, BW.Num(), 3);
		UTEST_EQUAL(Name, BW[0].GetBoneIndex(), 0);
		UTEST_EQUAL(Name, BW[0].GetWeight(), 0.7f);
		UTEST_EQUAL(Name, BW[1].GetBoneIndex(), 1);
		UTEST_EQUAL(Name, BW[1].GetWeight(), 0.5f);
		UTEST_EQUAL(Name, BW[2].GetBoneIndex(), 2);
		UTEST_EQUAL(Name, BW[2].GetWeight(), 0.075f);
	}

	{
		const TCHAR* Name = TEXT("FBoneWeights::Blend[Two/Three, Two Overlap]");

		auto BW1 = CreateWeights({ {0, 1.0f}, {1, 0.5f} });
		auto BW2 = CreateWeights({ {0, 0.5f}, {2, 0.5f}, {1, 0.25f} });

		FBoneWeightsSettings S;
		S.SetNormalizeType(EBoneWeightNormalizeType::None);
		S.SetBlendZeroInfluence(false);

		auto BW = FBoneWeights::Blend(BW1, BW2, 0.5f, S);

		UTEST_EQUAL(Name, BW.Num(), 3);
		UTEST_EQUAL(Name, BW[0].GetBoneIndex(), 0);
		UTEST_EQUAL(Name, BW[0].GetWeight(), 0.75f);
		UTEST_EQUAL(Name, BW[1].GetBoneIndex(), 2);
		UTEST_EQUAL(Name, BW[1].GetWeight(), 0.5f);
		UTEST_EQUAL(Name, BW[2].GetBoneIndex(), 1);
		UTEST_EQUAL(Name, BW[2].GetWeight(), 0.375f);

		S.SetBlendZeroInfluence(true);

		BW = FBoneWeights::Blend(BW1, BW2, 0.5f, S);

		UTEST_EQUAL(Name, BW.Num(), 3);
		UTEST_EQUAL(Name, BW[0].GetBoneIndex(), 0);
		UTEST_EQUAL(Name, BW[0].GetWeight(), 0.75f);
		UTEST_EQUAL(Name, BW[1].GetBoneIndex(), 1);
		UTEST_EQUAL(Name, BW[1].GetWeight(), 0.375f);
		UTEST_EQUAL(Name, BW[2].GetBoneIndex(), 2);
		UTEST_EQUAL(Name, BW[2].GetWeight(), 0.25f);
	}

	{
		const TCHAR* Name = TEXT("FBoneWeights::Blend[Barycentric, Two/Two/Three, Two Overlap]");

		auto BW1 = CreateWeights({ {0, 1.0f}, {1, 0.5f} });
		auto BW2 = CreateWeights({ {0, 0.5f}, {2, 0.5f} });
		auto BW3 = CreateWeights({ {0, 0.5f}, {1, 0.5f}, {3, 0.1f} });

		FBoneWeightsSettings S;
		S.SetNormalizeType(EBoneWeightNormalizeType::None);
		S.SetBlendZeroInfluence(true);
		auto BW = FBoneWeights::Blend(BW1, BW2, BW3, 0.2f, 0.3f, 0.5f, S);

		UTEST_EQUAL(Name, BW.Num(), 4);
		UTEST_EQUAL(Name, BW[0].GetBoneIndex(), 0);
		UTEST_EQUAL(Name, BW[0].GetWeight(), 0.6f);  // 0.2*1.0 + 0.3*0.5 + 0.5*0.5
		UTEST_EQUAL(Name, BW[1].GetBoneIndex(), 1);
		UTEST_EQUAL(Name, BW[1].GetWeight(), 0.35f); // 0.2*0.5 + 0.3*0.0 + 0.5*0.5
		UTEST_EQUAL(Name, BW[2].GetBoneIndex(), 2);
		UTEST_EQUAL(Name, BW[2].GetWeight(), 0.15f); // 0.2*0.0 + 0.3*0.5 + 0.5*0.0
		UTEST_EQUAL(Name, BW[3].GetBoneIndex(), 3);
		UTEST_EQUAL(Name, BW[3].GetWeight(), 0.05f); // 0.2*0.0 + 0.3*0.0 + 0.5*0.1
	}

	return true;
}


#endif //WITH_DEV_AUTOMATION_TESTS
