// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformNoScale.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformNoScale)

// Identity transform
const FTransformNoScale FTransformNoScale::Identity(FVector(0.0f), FQuat::Identity);
