// Copyright Epic Games, Inc. All Rights Reserved.

#include "TwoBoneIK.h"

namespace AnimationCore
{

void SolveTwoBoneIK(FTransform& InOutRootTransform, FTransform& InOutJointTransform, FTransform& InOutEndTransform, const FVector& JointTarget, const FVector& Effector, bool bAllowStretching, double StartStretchRatio, double MaxStretchScale)
{
	double LowerLimbLength = (InOutEndTransform.GetLocation() - InOutJointTransform.GetLocation()).Size();
	double UpperLimbLength = (InOutJointTransform.GetLocation() - InOutRootTransform.GetLocation()).Size();
	SolveTwoBoneIK(InOutRootTransform, InOutJointTransform, InOutEndTransform, JointTarget, Effector, UpperLimbLength, LowerLimbLength, bAllowStretching, StartStretchRatio, MaxStretchScale);
}

void SolveTwoBoneIK(FTransform& InOutRootTransform, FTransform& InOutJointTransform, FTransform& InOutEndTransform, const FVector& JointTarget, const FVector& Effector, double UpperLimbLength, double LowerLimbLength, bool bAllowStretching, double StartStretchRatio, double MaxStretchScale)
{
	FVector OutJointPos, OutEndPos;

	FVector RootPos = InOutRootTransform.GetLocation();
	FVector JointPos = InOutJointTransform.GetLocation();
	FVector EndPos = InOutEndTransform.GetLocation();

	// IK solver
	AnimationCore::SolveTwoBoneIK(RootPos, JointPos, EndPos, JointTarget, Effector, OutJointPos, OutEndPos, UpperLimbLength, LowerLimbLength, bAllowStretching, StartStretchRatio, MaxStretchScale);

	// Update transform for upper bone.
	{
		// Get difference in direction for old and new joint orientations
		FVector const OldDir = (JointPos - RootPos).GetSafeNormal();
		FVector const NewDir = (OutJointPos - RootPos).GetSafeNormal();
		// Find Delta Rotation take takes us from Old to New dir
		FQuat const DeltaRotation = FQuat::FindBetweenNormals(OldDir, NewDir);
		// Rotate our Joint quaternion by this delta rotation
		InOutRootTransform.SetRotation(DeltaRotation * InOutRootTransform.GetRotation());
		// And put joint where it should be.
		InOutRootTransform.SetTranslation(RootPos);

	}

	// update transform for middle bone
	{
		// Get difference in direction for old and new joint orientations
		FVector const OldDir = (EndPos - JointPos).GetSafeNormal();
		FVector const NewDir = (OutEndPos - OutJointPos).GetSafeNormal();

		// Find Delta Rotation take takes us from Old to New dir
		FQuat const DeltaRotation = FQuat::FindBetweenNormals(OldDir, NewDir);
		// Rotate our Joint quaternion by this delta rotation
		InOutJointTransform.SetRotation(DeltaRotation * InOutJointTransform.GetRotation());
		// And put joint where it should be.
		InOutJointTransform.SetTranslation(OutJointPos);

	}

	// Update transform for end bone.
	// currently not doing anything to rotation
	// keeping input rotation
	// Set correct location for end bone.
	InOutEndTransform.SetTranslation(OutEndPos);
}

void SolveTwoBoneIK(const FVector& RootPos, const FVector& JointPos, const FVector& EndPos, const FVector& JointTarget, const FVector& Effector, FVector& OutJointPos, FVector& OutEndPos, bool bAllowStretching, double StartStretchRatio, double MaxStretchScale)
{
	const double LowerLimbLength = (EndPos - JointPos).Size();
	const double UpperLimbLength = (JointPos - RootPos).Size();

	SolveTwoBoneIK(RootPos, JointPos, EndPos, JointTarget, Effector, OutJointPos, OutEndPos, UpperLimbLength, LowerLimbLength, bAllowStretching, StartStretchRatio, MaxStretchScale);
}

void SolveTwoBoneIK(const FVector& RootPos, const FVector& JointPos, const FVector& EndPos, const FVector& JointTarget, const FVector& Effector, FVector& OutJointPos, FVector& OutEndPos, double UpperLimbLength, double LowerLimbLength, bool bAllowStretching, double StartStretchRatio, double MaxStretchScale)
{
	// This is our reach goal.
	FVector DesiredPos = Effector;
	FVector DesiredDelta = DesiredPos - RootPos;
	double DesiredLength = DesiredDelta.Size();

	// Find lengths of upper and lower limb in the ref skeleton.
	// Use actual sizes instead of ref skeleton, so we take into account translation and scaling from other bone controllers.
	double MaxLimbLength = LowerLimbLength + UpperLimbLength;

	// Check to handle case where DesiredPos is the same as RootPos.
	FVector	DesiredDir;
	if (DesiredLength < DOUBLE_KINDA_SMALL_NUMBER)
	{
		DesiredLength = DOUBLE_KINDA_SMALL_NUMBER;
		DesiredDir = FVector(1, 0, 0);
	}
	else
	{
		DesiredDir = DesiredDelta.GetSafeNormal();
	}

	// Get joint target (used for defining plane that joint should be in).
	FVector JointTargetDelta = JointTarget - RootPos;
	const double JointTargetLengthSqr = JointTargetDelta.SizeSquared();

	// Same check as above, to cover case when JointTarget position is the same as RootPos.
	FVector JointPlaneNormal, JointBendDir;
	if (JointTargetLengthSqr < FMath::Square(DOUBLE_KINDA_SMALL_NUMBER))
	{
		JointBendDir = FVector(0, 1, 0);
		JointPlaneNormal = FVector(0, 0, 1);
	}
	else
	{
		JointPlaneNormal = DesiredDir ^ JointTargetDelta;

		// If we are trying to point the limb in the same direction that we are supposed to displace the joint in, 
		// we have to just pick 2 random vector perp to DesiredDir and each other.
		if (JointPlaneNormal.SizeSquared() < FMath::Square(DOUBLE_KINDA_SMALL_NUMBER))
		{
			DesiredDir.FindBestAxisVectors(JointPlaneNormal, JointBendDir);
		}
		else
		{
			JointPlaneNormal.Normalize();

			// Find the final member of the reference frame by removing any component of JointTargetDelta along DesiredDir.
			// This should never leave a zero vector, because we've checked DesiredDir and JointTargetDelta are not parallel.
			JointBendDir = JointTargetDelta - ((JointTargetDelta | DesiredDir) * DesiredDir);
			JointBendDir.Normalize();
		}
	}

	//UE_LOGF(LogAnimationCore, Log, "UpperLimb : %0.2f, LowerLimb : %0.2f, MaxLimb : %0.2f", UpperLimbLength, LowerLimbLength, MaxLimbLength);

	if (bAllowStretching)
	{
		const double ScaleRange = MaxStretchScale - StartStretchRatio;
		if (ScaleRange > DOUBLE_KINDA_SMALL_NUMBER && MaxLimbLength > DOUBLE_KINDA_SMALL_NUMBER)
		{
			const double ReachRatio = DesiredLength / MaxLimbLength;
			const double ScalingFactor = (MaxStretchScale - 1.0) * FMath::Clamp((ReachRatio - StartStretchRatio) / ScaleRange, 0.0, 1.0);
			if (ScalingFactor > DOUBLE_KINDA_SMALL_NUMBER)
			{
				LowerLimbLength *= (1.0 + ScalingFactor);
				UpperLimbLength *= (1.0 + ScalingFactor);
				MaxLimbLength *= (1.0 + ScalingFactor);
			}
		}
	}

	OutEndPos = DesiredPos;
	OutJointPos = JointPos;

	// If we are trying to reach a goal beyond the length of the limb, clamp it to something solvable and extend limb fully.
	if (DesiredLength >= MaxLimbLength)
	{
		OutEndPos = RootPos + (MaxLimbLength * DesiredDir);
		OutJointPos = RootPos + (UpperLimbLength * DesiredDir);
	}
	else
	{
		// So we have a triangle we know the side lengths of. We can work out the angle between DesiredDir and the direction of the upper limb
		// using the sin rule:
		const double TwoAB = 2.0 * UpperLimbLength * DesiredLength;

		const double CosAngle = (TwoAB != 0.0) ? ((UpperLimbLength*UpperLimbLength) + (DesiredLength*DesiredLength) - (LowerLimbLength*LowerLimbLength)) / TwoAB : 0.0;

		// If CosAngle is less than 0, the upper arm actually points the opposite way to DesiredDir, so we handle that.
		const bool bReverseUpperBone = (CosAngle < 0.0);

		// Angle between upper limb and DesiredDir
		// ACos clamps internally so we dont need to worry about out-of-range values here.
		const double Angle = FMath::Acos(CosAngle);

		// Now we calculate the distance of the joint from the root -> effector line.
		// This forms a right-angle triangle, with the upper limb as the hypotenuse.
		const double JointLineDist = UpperLimbLength * FMath::Sin(Angle);

		// And the final side of that triangle - distance along DesiredDir of perpendicular.
		// ProjJointDistSqr can't be neg, because JointLineDist must be <= UpperLimbLength because appSin(Angle) is <= 1.
		const double ProjJointDistSqr = (UpperLimbLength*UpperLimbLength) - (JointLineDist*JointLineDist);
		// although this shouldn't be ever negative, sometimes Xbox release produces -0.f, causing ProjJointDist to be NaN
		// so now I branch it. 						
		double ProjJointDist = (ProjJointDistSqr > 0.0) ? FMath::Sqrt(ProjJointDistSqr) : 0.0;
		if (bReverseUpperBone)
		{
			ProjJointDist *= -1.f;
		}

		// So now we can work out where to put the joint!
		OutJointPos = RootPos + (ProjJointDist * DesiredDir) + (JointLineDist * JointBendDir);
	}
}

}