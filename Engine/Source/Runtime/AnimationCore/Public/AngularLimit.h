// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AngularLimit.h: Angular limit features
=============================================================================*/

#pragma once

#include "CommonAnimTypes.h"
#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "Math/Quat.h"
#include "UObject/ObjectMacros.h"

namespace AnimationCore
{
	ANIMATIONCORE_API bool ConstrainAngularRangeUsingEuler(FQuat& InOutQuatRotation, const FQuat& InRefRotation, const FVector& InLimitMinDegrees, const FVector& InLimitMaxDegrees);
}