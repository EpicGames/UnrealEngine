// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationCore.h: Render core module definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAnimationCore, Log, All);