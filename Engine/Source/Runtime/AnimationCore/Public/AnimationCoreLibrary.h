// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationCoreLibrary.h: Render core module definitions.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EulerTransform.h"
#include "Math/MathFwd.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/Vector.h"

class FName;
struct FConstraintData;
struct FTransformConstraint;

DECLARE_DELEGATE_RetVal_OneParam(FTransform, FGetGlobalTransform, FName);

namespace AnimationCore
{
	/**
	 * Resolve Constraints based on input constraints data and current transform
	 * 
	 * @param	CurrentTransform		Current transform, based on BaseTransform
	 * @param	BaseTransform			Base transform of the current transform, where Constraint transform would be converted
	 * @param	Constraints				List of constraints to be used by this transform 
	 * @param	OnGetGlobalTransform	Delegate to get transform data for constraints
	 *
	 * @return resolved transform 
	 */
	ANIMATIONCORE_API FTransform SolveConstraints(const FTransform& CurrentTransform, const FTransform& BaseTransform, const TArray<FTransformConstraint>& Constraints, const FGetGlobalTransform& OnGetGlobalTransform);
	/**
	 * Resolve Constraints based on input constraints data and current transform
	 * 
	 * @param	CurrentTransform		Current transform, based on BaseTransform
	 * @param	BaseTransform			Base transform of the current transform, where Constraint transform would be converted
	 * @param	Constraints				List of constraints - should contains latest transform
	 *
	 * @return resolved transform 
	 */
	ANIMATIONCORE_API FTransform SolveConstraints(const FTransform& CurrentTransform, const FTransform& CurrentParentTransform, const TArray<FConstraintData>& Constraints);
	/** 
	 * Aim solver
	 *
	 * This solves new transform that aims at the target based on inputs
	 *
	 * @param	CurrentTransform	Current Transform
	 * @param	TargetPosition		Target to look at
	 * @param	AimVector			Aim vector in Current Transform
	 * @param	bUseUpVector		Whether or not to use Up vector
	 * @param	UpVector			Up Vector in Current Transform if bUseUpVector is true
	 * @param	AimClampInDegree	Clamp cone around the AimVector
	 *
	 * @return  Delta Rotation to turn
	 */
	ANIMATIONCORE_API FQuat SolveAim(const FTransform& CurrentTransform, const FVector& TargetPosition, const FVector& AimVector, bool bUseUpVector = false, const FVector& UpVector = FVector::UpVector, float AimClampInDegree = 0.f);

	/**
	 * Converts a euler rotation represented by a vector of rotations in degrees and a rotation order to a quaternion
	 * bUseUEHandyness If True will use UE handyness with right handed X and Y angles left handed Z, if False all are left handed
	 */
	ANIMATIONCORE_API FQuat QuatFromEuler(const FVector& XYZAnglesInDegrees, EEulerRotationOrder RotationOrderr = EEulerRotationOrder::ZYX, bool bUseUEHandyness = false);

	/**
	* Converts a quaternion to a euler rotation represented by a vector of rotations in degrees and a rotation order
	* bUseUEHandyness If True will use UE handyness with right handed X and Y angles left handed Z, if False all are left handed
	*/
	ANIMATIONCORE_API FVector EulerFromQuat(const FQuat& Rotation, EEulerRotationOrder RotationOrder = EEulerRotationOrder::ZYX, bool bUseUEHandyness = false);

	/**
	 * Converts a euler rotation from one rotation order to another
	 */
	ANIMATIONCORE_API FVector ChangeEulerRotationOrder(const FVector& XYZAnglesInDegrees, EEulerRotationOrder SourceRotationOrder, EEulerRotationOrder TargetRotationOrder, bool bUseUEHandyness = false);
}
