// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Templates/Casts.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "AnimationDataSource.generated.h"

class UClass;

UCLASS(MinimalAPI)
class UAnimationDataSourceRegistry : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Registers a new data source under a given name.
	 * Returns false in case the data source has already been registered.
	 *
	 * @param InName         The name to register the data source under
	 * @param InDataSource   The data source to register
	 * @return true if succeeded
	 */
	ANIMATIONCORE_API bool RegisterDataSource(const FName& InName, UObject* InDataSource);

	/**
	 * Unregisters / removes an existing data source under a given name.
	 * Returns false in case the data source has already been unregistered.
	 *
	 * @param InName The name of the data source to remove
	 * @return true if succeeded
	 */
	ANIMATIONCORE_API bool UnregisterDataSource(const FName& InName);

	/**
	 * Returns true if this registry contains a source with the given name
	 *
	 * @param InName The name of the data source to look up.
	 * @return true if a source with the given name exists
	 */
	ANIMATIONCORE_API bool ContainsSource(const FName& InName) const;

	/**
	 * Returns a given data source and cast it to the expected class.
	 *
	 * @param InName The name of the data source to look up.
	 * @param InExpectedClass The class expected from the data source
	 * @return The requested data source
	 */
	ANIMATIONCORE_API UObject* RequestSource(const FName& InName, UClass* InExpectedClass) const;

	/**
	 * Returns a given data source and cast it to the expected class.
	 * 
	 * @param InName The name of the data source to look up.
	 * @return The requested data source
	 */
	template<class T>
	T* RequestSource(const FName& InName) const
	{
		UObject* DataSource = RequestSource(InName, T::StaticClass());
		if (DataSource == nullptr)
		{
			return nullptr;
		}
		return Cast<T>(DataSource);
	}

private:

	UPROPERTY(transient)
	TMap<FName, TWeakObjectPtr<UObject>> DataSources;

	/** Clear Invalid Data Sources that are GC-ed */
	ANIMATIONCORE_API void ClearInvalidDataSource();
};
