// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

typedef uint16 FBoneIndexType;

// Important: only use this define when comparing FBoneIndexType (16 bit), it should not be used to compare 32 bit bone indicies stored in FBoneIndexBase
#define INVALID_BONE_INDEX FBoneIndexType(INDEX_NONE) 


struct FBoneIndexBase
{
	FBoneIndexBase() : BoneIndex(INDEX_NONE) {}

	inline int32 GetInt() const { return BoneIndex; }
	inline FBoneIndexType GetBoneIndex() const { return static_cast<FBoneIndexType>(BoneIndex); }

	inline bool IsRootBone() const { return BoneIndex == 0; }

	inline bool IsValid() const { return BoneIndex != INDEX_NONE; }

	inline explicit operator int32() const { return BoneIndex; }

	inline explicit operator bool() const { return IsValid(); }

	friend inline uint32 GetTypeHash(const FBoneIndexBase& Index) { return GetTypeHash(Index.BoneIndex); }

protected:
	int32 BoneIndex;
};

inline int32 GetIntFromComp(const int32 InComp)
{
	return InComp;
}

inline int32 GetIntFromComp(const FBoneIndexBase& InComp)
{
	return InComp.GetInt();
}

#define UE_BONE_INDEX_COMPARE_OPERATORS(LhsType, RhsType) \
	inline friend bool operator==(LhsType Lhs, RhsType Rhs) { return GetIntFromComp(Lhs) == GetIntFromComp(Rhs); } \
	inline friend bool operator>(LhsType Lhs, RhsType Rhs) { return GetIntFromComp(Lhs) > GetIntFromComp(Rhs); } \
	inline friend bool operator>=(LhsType Lhs, RhsType Rhs) { return GetIntFromComp(Lhs) >= GetIntFromComp(Rhs); } \
	inline friend bool operator<(LhsType Lhs, RhsType Rhs) { return GetIntFromComp(Lhs) < GetIntFromComp(Rhs); } \
	inline friend bool operator<=(LhsType Lhs, RhsType Rhs) { return GetIntFromComp(Lhs) <= GetIntFromComp(Rhs); } \

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	#define UE_BONE_INDEX_COMPARE_OPERATORS_INVERTED UE_BONE_INDEX_COMPARE_OPERATORS(const int32, const Type&)
#else
	#define UE_BONE_INDEX_COMPARE_OPERATORS_INVERTED
#endif

#define UE_BONE_INDEX_OPERATORS(Type) \
	UE_BONE_INDEX_COMPARE_OPERATORS(const Type&, const Type&) \
	UE_BONE_INDEX_COMPARE_OPERATORS(const Type&, const int32) \
	UE_BONE_INDEX_COMPARE_OPERATORS_INVERTED \
	Type& operator++() { ++BoneIndex; return *this; } \
	Type& operator--() { --BoneIndex; return *this; } \
	Type& operator=(const Type& Rhs) { BoneIndex = Rhs.BoneIndex; return *this; }

// This represents a compact pose bone index. A compact pose is held by a bone container and can have a different ordering than either the skeleton or skeletal mesh.
struct FCompactPoseBoneIndex : public FBoneIndexBase
{
public:
	FCompactPoseBoneIndex() = default;
	explicit FCompactPoseBoneIndex(int32 InBoneIndex) { BoneIndex = InBoneIndex; }
	explicit FCompactPoseBoneIndex(FBoneIndexType InBoneIndex) { BoneIndex = (InBoneIndex != INVALID_BONE_INDEX) ? InBoneIndex : INDEX_NONE; } // Handle 16bit -> 32bit
	UE_BONE_INDEX_OPERATORS(FCompactPoseBoneIndex)
};

// This represents a skeletal mesh bone index which may differ from the skeleton bone index it corresponds to.
struct FMeshPoseBoneIndex : public FBoneIndexBase
{
public:
	FMeshPoseBoneIndex() = default;
	explicit FMeshPoseBoneIndex(int32 InBoneIndex) { BoneIndex = InBoneIndex; }
	explicit FMeshPoseBoneIndex(FBoneIndexType InBoneIndex) { BoneIndex = (InBoneIndex != INVALID_BONE_INDEX) ? InBoneIndex : INDEX_NONE; } // Handle 16bit -> 32bit
	UE_BONE_INDEX_OPERATORS(FMeshPoseBoneIndex)
};

// This represents a skeleton bone index which may differ from the skeletal mesh bone index it corresponds to.
struct FSkeletonPoseBoneIndex : public FBoneIndexBase
{
public:
	FSkeletonPoseBoneIndex() = default;
	explicit FSkeletonPoseBoneIndex(int32 InBoneIndex) { BoneIndex = InBoneIndex; }
	explicit FSkeletonPoseBoneIndex(FBoneIndexType InBoneIndex) { BoneIndex = (InBoneIndex != INVALID_BONE_INDEX) ? InBoneIndex : INDEX_NONE; } // Handle 16bit -> 32bit
	UE_BONE_INDEX_OPERATORS(FSkeletonPoseBoneIndex)
};

template <typename ValueType>
struct TCompactPoseBoneIndexMapKeyFuncs : public TDefaultMapKeyFuncs<const FCompactPoseBoneIndex, ValueType, false>
{
	static inline FCompactPoseBoneIndex			GetSetKey(TPair<FCompactPoseBoneIndex, ValueType> const& Element) { return Element.Key; }
	static inline uint32							GetKeyHash(FCompactPoseBoneIndex const& Key) { return GetTypeHash(Key.GetInt()); }
	static inline bool								Matches(FCompactPoseBoneIndex const& A, FCompactPoseBoneIndex const& B) { return (A.GetInt() == B.GetInt()); }
};