// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/IsSorted.h"
#include "AnimationCore.h"
#include "BoneIndices.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "GPUSkinPublicDefs.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"

#include <limits>
#include <type_traits>

namespace UE { 
namespace AnimationCore {

/**
The maximum number of inline bone weights.
*/
static constexpr int32 MaxInlineBoneWeightCount = MAX_TOTAL_INFLUENCES;

/** The maximum raw weight value */
static constexpr uint16 MaxRawBoneWeight = std::numeric_limits<uint16>::max();

/** The maximum raw weight value as a float value */
static constexpr float MaxRawBoneWeightFloat = static_cast<float>(std::numeric_limits<uint16>::max());

/** The inverse of the maximum raw weight value as a float value. Used for scaling. */
static constexpr float InvMaxRawBoneWeightFloat = 1.0f / MaxRawBoneWeightFloat;

/** The threshold value at or above which a 0-1 normalized bone weight value will be stored as a non-zero value after
 *  scaling and quantizing to a 16-bit integer */
static constexpr float BoneWeightThreshold = InvMaxRawBoneWeightFloat;

class FBoneWeight
{
public:
	/**
	* Returns true if this object's bone weight values are equal to the other container's values.
	* @note Only equality comparison are supported. Relational comparisons are meaningless.
	*/
	bool operator==(const FBoneWeight& InBoneWeight) const
	{
		return BoneIndex == InBoneWeight.BoneIndex && RawWeight == InBoneWeight.RawWeight;
	}

	/**
	* Returns true if this object's bone weight values are not equal to the other container's values.
	* @note Only equality comparison are supported. Relational comparisons are meaningless.
	*/
	bool operator!=(const FBoneWeight& InBoneWeight) const
	{
		return BoneIndex != InBoneWeight.BoneIndex || RawWeight != InBoneWeight.RawWeight;
	}

	/**
	* The maximum raw weight value for a bone influence.
	*/
	static constexpr uint16 GetMaxRawWeight() 
	{
		return TNumericLimits<uint16>::Max();
	}

	/** A standard predicate we use for sorting by weight, in a descending order of weights */
	static bool DescSortByWeightPredicate(const FBoneWeight& A, const FBoneWeight& B)
	{
		return A.GetRawWeight() > B.GetRawWeight();
	}
	
	/** Default constructor. NOTE: The values are uninitialized. */
	FBoneWeight() = default;

	/**
	* A constructor for old-style bone weights where the weight was stored as an unsigned eight
	* bit integer.
	*/
	explicit FBoneWeight(FBoneIndexType InBoneIndex, uint8 InWeight)
	    : BoneIndex(InBoneIndex)
	{
		RawWeight = uint16( (InWeight << 8) | InWeight );
	}

	/**
	* A constructor for bone weights where the weight is stored as an unsigned sixteen
	* bit integer. This is the natural storage format for this container.
	*/
	explicit FBoneWeight(FBoneIndexType InBoneIndex, uint16 InRawWeight)
	    : BoneIndex(InBoneIndex), RawWeight(InRawWeight)
	{
	}

	/**
	* A constructor for bone weights which converts the weight from a float value in the
	* [0-1] range. Values outside of that range are clamped to the range.
	*/
	explicit FBoneWeight(FBoneIndexType InBoneIndex, float InWeight)
	    : BoneIndex(InBoneIndex)
	{
		SetWeight(InWeight);
	}

	/**
	* Set the bone stored index.
	*/
	void SetBoneIndex(FBoneIndexType InBoneIndex)
	{
		BoneIndex = InBoneIndex;
	}

	/**
	* Returns the bone stored index.
	*/
	FBoneIndexType GetBoneIndex() const
	{
		return BoneIndex;
	}

	/**
	* Set stored weight as a float. Values outside the [0-1] range are clamped to that range.
	* Undefined float values will result in an undefined weight.
	*/
	void SetWeight(float InWeight)
	{
		InWeight = FMath::Clamp(InWeight, 0.0f, 1.0f);
		RawWeight = static_cast<uint16>(InWeight * static_cast<float>(GetMaxRawWeight()) + 0.5f);
	}

	/**
	* Returns the stored weight value as a float in the [0-1] range.
	*/
	float GetWeight() const
	{
		return RawWeight / static_cast<float>(GetMaxRawWeight());
	}

	/**
	* Set the stored weight in the raw format. This avoids any floating point conversion.
	*/
	void SetRawWeight(const uint16 InRawWeight)
	{
		RawWeight = InRawWeight;
	}

	/**
	* Returns the stored weight in the container's raw format, avoiding floating point
	* conversion.
	*/
	uint16 GetRawWeight() const
	{
		return RawWeight;
	}

	void Serialize(FArchive& InArchive)
	{
		InArchive << BoneIndex;
		InArchive << RawWeight;
	}

	/** Returns a hash value from the bone weight values */
	uint32 GetTypeHash() const
	{
		return HashCombine(::GetTypeHash(static_cast<uint16>(BoneIndex)), ::GetTypeHash(RawWeight));
	}

	/**
	* A helper function to return a human-readable version of the bone weight.
	*/
	FString ToString() const
	{
		return FString::Printf(TEXT("<%d, %g>"), BoneIndex, GetWeight());
	}


	/**
	* Returns the contents of this object as an int32.
	*/
	int32 ToInt32() const
	{
		return static_cast<int32>(static_cast<uint16>(BoneIndex) << 16 | RawWeight);
	}

	/**
	* Converts an int32 to an FBoneWeight object and returns it.
	*/
	static FBoneWeight FromInt32(int32 InBoneWeight)
	{
		FBoneWeight BW;
		BW.RawWeight = static_cast<uint16>(static_cast<uint32>(InBoneWeight) & 0xFFFFU);
		BW.BoneIndex = static_cast<FBoneIndexType>(static_cast<uint32>(InBoneWeight) >> 16);
		return BW;
	}

private:
	FBoneIndexType BoneIndex;
	uint16 RawWeight;
};

static_assert(sizeof(FBoneWeight) == sizeof(int32), "FBoneWeight must be 32-bits");

/** Specifies the method for which the bone weights are normalized after the bone weight
  * list is modified.
  */
enum class EBoneWeightNormalizeType
{
	/** No normalization is performed. The sum of the weight values can exceed 1.0 */
	None,

	/** Normalization is only performed if the sum of the weights exceeds 1.0 */
	AboveOne,

	/** Normalization is always performed such that the sum of the weights is always equal to 1.0 */
	Always
};


class FBoneWeightsSettings
{
public:
	FBoneWeightsSettings() = default;

	/** Set the normalization type when manipulating the weight values in FBoneWeights */
	FBoneWeightsSettings& SetNormalizeType(EBoneWeightNormalizeType InNormalizeType)
	{
		NormalizeType = InNormalizeType;
		return *this;
	}

	/** Returns the current normalization type for these settings */
	EBoneWeightNormalizeType GetNormalizeType() const { return NormalizeType; }

	/** Sets the maximum number of weights that can be applied to a single FBoneWeights object.
	  * When weights are added, the smallest weights, past this limit, are dropped.
	  */
	FBoneWeightsSettings& SetMaxWeightCount(int32 InMaxWeightCount)
	{
		MaxWeightCount = FMath::Max(1, InMaxWeightCount);
		return *this;
	}

	/** Returns the maximum number of weights for these settings */
	int32 GetMaxWeightCount() const { return MaxWeightCount; }

	/** Sets the minimum influence allowed. Any bone weight value below this threshold value is
	  * discarded. The threshold value is clamped to the half-closed interval (0, 1] since
	  * weight values of zero indicate no influence at all and are ignored completely.
	  * By default, all non-zero weights are allowed.
	  */
	FBoneWeightsSettings& SetWeightThreshold(float InWeightThreshold)
	{
		InWeightThreshold = FMath::Clamp(InWeightThreshold, 0.0f, 1.0f);
		WeightThreshold = static_cast<uint16>(InWeightThreshold * FBoneWeight::GetMaxRawWeight() + 0.5f);
		WeightThreshold = FMath::Max(WeightThreshold, static_cast<uint16>(1));
		return *this;
	}

	/** Returns the weight threshold as a float value between (0, 1]. */
	float GetWeightThreshold() const
	{
		return WeightThreshold / static_cast<float>(TNumericLimits<uint16>::Max());
	}

	/** Returns the raw weight threshold. This is the value used internally for weight
	  * computation.
	  */
	uint16 GetRawWeightThreshold() const
	{
		return WeightThreshold;
	}

	/** Set the default bone index to use if no weights were set. This can be used to ensure
	    that there's always a valid weight applied to a skinned vertex. */
	void SetDefaultBoneIndex(FBoneIndexType InBoneIndex)
	{
		DefaultBoneIndex = InBoneIndex;
		bHasDefaultBoneIndex = true;
	}

	/** Returns the current default bone index to set if no weights were set. If no default
	    bone index has been set, then this value is undefined. Call HasDefaultBoneIndex to
		check */
	FBoneIndexType GetDefaultBoneIndex() const
	{
		return DefaultBoneIndex;
	}

	/** Clears the default bone index. This allows bone weights arrays to be empty if no weights
		are set. */
	void ClearDefaultBoneIndex()
	{
		bHasDefaultBoneIndex = false;
	}

	/** Returns \c true if a default bone index should be applied in the absence of other
	    weights. */
	bool HasDefaultBoneIndex() const
	{
		return bHasDefaultBoneIndex;
	}

	/** Set the way we treat the bones with zero influence during the blend. If true, we assume all 
	  * bones with no influence have a zero weight (default) and participate in the blend calculation.
	  * If false, we ignore the bone during the blend.
	  */
	void SetBlendZeroInfluence(bool bInBlendZeroInfluence) 
	{
		bBlendZeroInfluence = bInBlendZeroInfluence;
	}

	/** Return true if bones with no influence are considered to have a zero weight during the 
	  * blend calculation. */
	bool GetBlendZeroInfluence() const 
	{
		return bBlendZeroInfluence;
	}

private:
	EBoneWeightNormalizeType NormalizeType = EBoneWeightNormalizeType::Always;
	int32 MaxWeightCount = MaxInlineBoneWeightCount;
	uint16 WeightThreshold = 1; 
	FBoneIndexType DefaultBoneIndex = static_cast<FBoneIndexType>(0);
	bool bHasDefaultBoneIndex = false;
	bool bBlendZeroInfluence = true; 
};

/** A null adapter for a bone weight container to use with TBoneWeights. Use as a template to
    create adapters for other types of containers. */
struct FBoneWeightNullAdapter
{
	struct Empty {};
	using ContainerType = Empty;

	/** Set the number of elements to reserve in the container. The elements can be left 
	    in an uninitialized state. The TBoneWeights algorithms will ensure that all elements
		will be properly defined at the end of an operation. */
	static void SetNum(ContainerType& InContainer, int32 InNum)
	{
	}

	static int32 Num(const ContainerType& InContainer)
	{
		return INDEX_NONE;
	}

	static FBoneWeight Get(const ContainerType& InContainer, int32 InIndex)
	{
		return {};
	}

	static void Set(ContainerType& InContainer, int32 InIndex, FBoneWeight InBoneWeight)
	{
	}

	static void Add(ContainerType& InContainer, FBoneWeight InBoneWeight)
	{
	}

	static void Remove(ContainerType& InContainer, int32 InIndex)
	{
	}

	template<typename Predicate>
	static void Sort(ContainerType& InContainer, Predicate InPredicate)
	{
	}

	template<typename Predicate>
	static int32 IndexOf(const ContainerType& InContainer, Predicate InPredicate)
	{
		return INDEX_NONE;
	}
};


/** A templated collection of bone weights algorithms. Requires an adapter to work with a
    dynamically resizable container. */
template<typename ContainerAdapter>
class TBoneWeights
{
public:
	using ContainerType = typename ContainerAdapter::ContainerType;

	TBoneWeights(ContainerType& InContainer) :
		Container(InContainer)
	{ }

	template<typename OtherContainerAdapter, typename CT = ContainerType>
	inline typename std::enable_if<!std::is_const<CT>::value, void>::type
	SetBoneWeights(
		const TBoneWeights<OtherContainerAdapter> &InBoneWeights,
		const FBoneWeightsSettings& InSettings = {});

	template<typename CT = ContainerType>
    inline typename std::enable_if<!std::is_const<CT>::value, void>::type
	SetBoneWeights(
		TArrayView<const FBoneWeight> BoneWeights,
		const FBoneWeightsSettings& InSettings = {});

	template<typename CT = ContainerType>
	inline typename std::enable_if<!std::is_const<CT>::value, void>::type
	SetBoneWeights(
		const FBoneIndexType* InBones,
		const float* InInfluences,
		int32 NumEntries,
		const FBoneWeightsSettings& InSettings = {}
		);

	template<typename CT = ContainerType>
	inline typename std::enable_if<!std::is_const<CT>::value, void>::type
	SetBoneWeights(
		const FBoneIndexType InBones[MaxInlineBoneWeightCount],
		const uint16 InInfluences[MaxInlineBoneWeightCount],
		const FBoneWeightsSettings& InSettings = {});

	template<typename CT = ContainerType>
    inline typename std::enable_if<!std::is_const<CT>::value, bool>::type
	AddBoneWeight(
		FBoneWeight InBoneWeight,
		const FBoneWeightsSettings& InSettings = {}
		);

	template<typename CT = ContainerType>
    inline typename std::enable_if<!std::is_const<CT>::value, bool>::type
    RemoveBoneWeight(
		FBoneIndexType InBoneIndex,
		const FBoneWeightsSettings& InSettings = {}
		);

	template<typename CT = ContainerType>
    inline typename std::enable_if<!std::is_const<CT>::value, void>::type
    Renormalize(
		const FBoneWeightsSettings& InSettings = {}
		);

	/** Blend two bone weights together, making sure to add in every influence from both,
	  * using the given settings. The bias value should lie on the [0,1] interval. Values outside
	  * that range may give unwanted results. 
	  * NOTE: The current container can also be used as an input.
	  */
	template<typename ContainerTypeA, typename ContainerTypeB, typename CT = ContainerType>
    inline typename std::enable_if<!std::is_const<CT>::value, void>::type
    Blend(
		const TBoneWeights<ContainerTypeA>& InBoneWeightsA,
		const TBoneWeights<ContainerTypeB>& InBoneWeightsB,
		float InBias,
		const FBoneWeightsSettings& InSettings = {});

	int32 Num() const
	{
		return ContainerAdapter::Num(Container);
	}

	FBoneWeight Get(int32 Index) const
	{
		return ContainerAdapter::Get(Container, Index);
	}

	FBoneWeight operator[](int32 Index) const
	{
		return ContainerAdapter::Get(Container, Index);
	}

	int32 FindWeightIndexByBone(
		FBoneIndexType InBoneIndex
		) const
	{
		return ContainerAdapter::IndexOf(Container, [InBoneIndex](const FBoneWeight& BW) { return InBoneIndex == BW.GetBoneIndex(); });
	}

	int32 GetTypeHash() const
	{
		uint32 Hash = ::GetTypeHash(Num());
		for (int32 Index = 0; Index < Num(); Index++)
		{
			Hash = HashCombine(Hash, Get(Index).GetTypeHash());
		}
		return Hash;
	}

	FString ToString() const
	{
		FString Result("[");
		if (Num())
		{
			Result += Get(0).ToString();
			for (int32 Index = 1; Index < Num(); Index++)
			{
				Result += TEXT(", ");
				Result += Get(Index).ToString();
			}
		}
		Result.Append(TEXT("]"));
		return Result;
	}

private:
	inline void SetBoneWeightsInternal(
		TArrayView<FBoneWeight> BoneWeights,
		const FBoneWeightsSettings& InSettings = {});

	inline void SortWeights();
	inline bool CullWeights(const FBoneWeightsSettings& InSettings);
	inline void NormalizeWeights(EBoneWeightNormalizeType InNormalizeType);

	inline bool Verify() const;

	// The externally owned container we're operating on.
	ContainerType &Container;
};


/// A simple container for per-vertex influence of bones and their weights.
class FBoneWeights
{
	using BoneWeightsTempAllocatorT = TInlineAllocator<MaxInlineBoneWeightCount>;
	using BoneWeightArrayT = TArray<FBoneWeight, BoneWeightsTempAllocatorT>;

	template<typename T>
	struct TArrayAdapter
	{
		using ContainerType = T;

		static void SetNum(ContainerType& InContainer, int32 InNum)
		{
			InContainer.SetNumUninitialized(InNum, EAllowShrinking::No);
		}

		static int32 Num(const ContainerType& InContainer)
		{
			return InContainer.Num();
		}

		static FBoneWeight Get(const ContainerType& InContainer, int32 InIndex)
		{
			return InContainer.GetData()[InIndex];
		}

		static void Set(ContainerType& InContainer, int32 InIndex, FBoneWeight InBoneWeight)
		{
			InContainer.GetData()[InIndex] = InBoneWeight;
		}

		static void Add(ContainerType& InContainer, FBoneWeight InBoneWeight)
		{
			InContainer.Add(InBoneWeight);
		}

		static void Remove(ContainerType& InContainer, int32 InIndex)
		{
			InContainer.RemoveAt(InIndex);
		}

		template<typename Predicate>
		static void Sort(ContainerType& InContainer, Predicate InPredicate)
		{
			InContainer.Sort(InPredicate);
		}

		template<typename Predicate>
		static int32 IndexOf(const ContainerType& InContainer, Predicate InPredicate)
		{
			return InContainer.IndexOfByPredicate(InPredicate);
		}
	};
	using FArrayWrapper = TBoneWeights<TArrayAdapter<BoneWeightArrayT>>;

public:
	FBoneWeights() = default;

	/**
	  * Returns true if all of this container's values and count are equal to the other
	  * container's values and count.
	  * @note Only equality comparison are supported. Relational comparisons are meaningless.
	  */
	bool operator==(const FBoneWeights& InBoneWeight) const
	{
		return BoneWeights == InBoneWeight.BoneWeights;
	}

	/**
	* Returns true if this container's values are equal to the other container's values.
	  @note Only equality comparison are supported. Relational comparisons are meaningless.
	*/
	bool operator!=(const FBoneWeights& InBoneWeight) const
	{
		return BoneWeights != InBoneWeight.BoneWeights;
	}

	/** Set a new bone weight. If an existing weight exists with the same bone index, it's
  	  * weight value is replaced with the weight value of the given entry. Otherwise a 
      * new index is added. In both cases the new entry is subject to the given settings, which
	  * may influence what the entry's final weight value is and whether it gets thrown away
      * for not passing the threshold value. If the weight was successfully incorporated, then
	  * this function returns true. Otherwise it returns false.
	  */
	inline bool SetBoneWeight(
	    FBoneWeight InBoneWeight,
	    const FBoneWeightsSettings& InSettings = {});

	bool SetBoneWeight(
	    FBoneIndexType InBone,
	    float InWeight,
	    const FBoneWeightsSettings& InSettings = {})
	{
		return SetBoneWeight(FBoneWeight{InBone, InWeight}, InSettings);
	}

	/** Removes a specific bone from the list of weights, re-normalizing and pruning bones,
	  * if needed.
	  */
	inline bool RemoveBoneWeight(
	    FBoneIndexType InBone,
	    const FBoneWeightsSettings& InSettings = {});

	/** Force normalization of weights. This is useful if a set of operations were performed
	  * with no normalization, for efficiency, and normalization is needed post-operation.
	  */
	inline void Renormalize(const FBoneWeightsSettings& InSettings = {});

	/** A helper to create a FBoneWeights container from FSoftSkinVertex data structure. */
	static inline FBoneWeights Create(
	    const FBoneIndexType InBones[MaxInlineBoneWeightCount],
	    const uint16 InWeights[MaxInlineBoneWeightCount],
	    const FBoneWeightsSettings& InSettings = {});

	/** A helper to create a FBoneWeights container from separated bone index and weight arrays.
	  * The size of the two arrays *must* be the same -- otherwise the behavior is undefined.
	  */
	static inline FBoneWeights Create(
	    const FBoneIndexType* InBones,
	    const float* InWeights,
	    int32 NumEntries,
	    const FBoneWeightsSettings& InSettings = {});

	/** A helper to create FBoneWeights container from a TArray of FBoneWeight objects. */
	static inline FBoneWeights Create(
	    TArrayView<const FBoneWeight> BoneWeights,
	    const FBoneWeightsSettings& InSettings = {});

	/** A helper to create FBoneWeights container from a TBoneWeights derivative object */
	template<typename OtherContainerAdapter>
	static FBoneWeights Create(
		TBoneWeights<OtherContainerAdapter> InBoneWeights,
		const FBoneWeightsSettings& InSettings = {})
	{
		FBoneWeights Result;
		FArrayWrapper W(Result.BoneWeights);
		W.SetBoneWeights(InBoneWeights, InSettings);
		return Result;
	}
	
	/** Blend two bone weights together, making sure to add in every influence from both,
	  * using the given settings. The bias value should lie on the [0,1] interval. Values outside
	  * that range may give unwanted results. 
	  */
	static inline FBoneWeights Blend(
	    const FBoneWeights& InA,
	    const FBoneWeights& InB,
	    float InBias,
	    const FBoneWeightsSettings& InSettings = {});

	/** Blend three bone weights via barycentric coordinates using the given settings. 
	  * Considers the influence from all of the bones. Coordinates should sum to one. 
	  */
	static inline FBoneWeights Blend(
	    const FBoneWeights& InA,
	    const FBoneWeights& InB,
		const FBoneWeights& InC,
	    float InBaryX,
		float InBaryY,
		float InBaryZ,
	    const FBoneWeightsSettings& InSettings = {});
	
	// Ranged-based for loop compatibility -- but only the const version.
	using RangedForConstIteratorType = BoneWeightArrayT::RangedForConstIteratorType;

	RangedForConstIteratorType begin() const { return BoneWeights.begin(); }
	RangedForConstIteratorType end() const { return BoneWeights.end(); }

	/** Return the number of bone weights in this container.
	  * @return The number of bone weights.
	  */
	int32 Num() const { return BoneWeights.Num(); }

	/** Return the weight at index Index. Using an index value less than zero -- or equal
	  * to or greater than the result of Num() -- is an undefined operation.
	  * @param Index The index of the desired bone weight value.
	  * @return The bone weight value for that index, or undefined if outside the valid range.
	  */
	const FBoneWeight& operator[](int32 Index) const { return BoneWeights.operator[](Index); }

	/** Returns the FBoneWeight list as an array view */
	TArrayView<const FBoneWeight> ToArrayView() const
	{
		return BoneWeights;
	}
	
	// -- Helper functions

	/** Find the bone weight corresponding to the given bone index. If a bone weight with this
	  * bone index does not exist it, a value of INDEX_NONE is returned.
	  */
	int32 FindWeightIndexByBone(FBoneIndexType InBoneIndex) const
	{
		return BoneWeights.IndexOfByPredicate(
		    [InBoneIndex](const FBoneWeight& BW) { return InBoneIndex == BW.GetBoneIndex(); });
	}

	void Serialize(FArchive& InArchive)
	{
		InArchive << BoneWeights;
	}

	int32 GetTypeHash() const
	{
		uint32 Hash = ::GetTypeHash(BoneWeights.Num());
		for (const FBoneWeight& BoneWeight : BoneWeights)
		{
			Hash = HashCombine(Hash, BoneWeight.GetTypeHash());
		}
		return Hash;
	}

	FString ToString() const
	{
		FString Result("[");
		Result.Append(FString::JoinBy(BoneWeights, TEXT(", "), [](const FBoneWeight& V) { return V.ToString(); }));
		Result.Append(TEXT("]"));
		return Result;
	}

private:
	friend uint32 GetTypeHash(const FBoneWeights& InBoneWeights);

	/// List of bone weights, in order of descending weight.
	BoneWeightArrayT BoneWeights;
};


/// TBoneWeights implementation
template<typename ContainerAdapter>
template<typename OtherContainerAdapter, typename CT>
typename std::enable_if<!std::is_const<CT>::value, void>::type
TBoneWeights<ContainerAdapter>::SetBoneWeights(
	const TBoneWeights<OtherContainerAdapter>& InBoneWeights,
	const FBoneWeightsSettings& InSettings /*= {}*/)
{
	TArray<FBoneWeight, TInlineAllocator<MaxInlineBoneWeightCount>> StackBoneWeights;
	StackBoneWeights.Reserve(InBoneWeights.Num());
	for (int32 Index = 0; Index < InBoneWeights.Num(); Index++)
	{
		FBoneWeight BW = InBoneWeights.Get(Index);
		if (BW.GetRawWeight() >= InSettings.GetRawWeightThreshold())
		{
			StackBoneWeights.Add(BW);
		}
	}	

	SetBoneWeightsInternal(StackBoneWeights, InSettings);
}

template<typename ContainerAdapter>
template<typename CT>
typename std::enable_if<!std::is_const<CT>::value, void>::type
TBoneWeights<ContainerAdapter>::SetBoneWeights(
	TArrayView<const FBoneWeight> InBoneWeights,
	const FBoneWeightsSettings& InSettings /*= {}*/)
{
	TArray<FBoneWeight, TInlineAllocator<MaxInlineBoneWeightCount>> StackBoneWeights;

	StackBoneWeights.Reserve(InBoneWeights.Num());
	for (const FBoneWeight& BW : InBoneWeights)
	{
		if (BW.GetRawWeight() >= InSettings.GetRawWeightThreshold())
		{
			StackBoneWeights.Add(BW);
		}
	}

	SetBoneWeightsInternal(StackBoneWeights, InSettings);
}


template<typename ContainerAdapter>
template<typename CT>
typename std::enable_if<!std::is_const<CT>::value, void>::type
TBoneWeights<ContainerAdapter>::SetBoneWeights(
	const FBoneIndexType* InBones,
	const float* InInfluences,
	int32 NumEntries,
	const FBoneWeightsSettings& InSettings /*= {}*/)
{
	TArray<FBoneWeight, TInlineAllocator<MaxInlineBoneWeightCount>> StackBoneWeights;

	StackBoneWeights.Reserve(NumEntries);
	for (int32 Index = 0; Index < NumEntries; Index++)
	{
		FBoneWeight BW(InBones[Index], InInfluences[Index]);
		if (BW.GetRawWeight() >= InSettings.GetRawWeightThreshold())
		{
			StackBoneWeights.Add(BW);
		}
	}

	SetBoneWeightsInternal(StackBoneWeights, InSettings);
}


template<typename ContainerAdapter>
template<typename CT>
typename std::enable_if<!std::is_const<CT>::value, void>::type
TBoneWeights<ContainerAdapter>::SetBoneWeights(
	const FBoneIndexType InBones[MaxInlineBoneWeightCount],
	const uint16 InInfluences[MaxInlineBoneWeightCount],
	const FBoneWeightsSettings& InSettings /*= {}*/)
{
	// The weights are valid until the first zero influence.
	int32 NumWeights = 0;
	for (int32 Index = 0; Index < MaxInlineBoneWeightCount && InInfluences[Index]; Index++)
	{
		FBoneWeight BW(InBones[Index], InInfluences[Index]);
		if (BW.GetRawWeight() >= InSettings.GetRawWeightThreshold())
		{
			NumWeights++;
		}
	}

	ContainerAdapter::SetNum(Container, NumWeights);

	int32 WeightIndex = 0;
	for (int32 Index = 0; Index < MaxInlineBoneWeightCount && InInfluences[Index]; Index++)
	{
		FBoneWeight BW(InBones[Index], InInfluences[Index]);
		if (BW.GetRawWeight() >= InSettings.GetRawWeightThreshold())
		{
			ContainerAdapter::Set(Container, WeightIndex++, BW);
		}
	}

	// Sort the weights by descending weight value before we clip it.
	SortWeights();

	if (WeightIndex > InSettings.GetMaxWeightCount())
	{
		ContainerAdapter::SetNum(Container, InSettings.GetMaxWeightCount());
	}

	NormalizeWeights(InSettings.GetNormalizeType());
}


template<typename ContainerAdapter>
void TBoneWeights<ContainerAdapter>::SetBoneWeightsInternal(
	TArrayView<FBoneWeight> BoneWeights,
	const FBoneWeightsSettings& InSettings /*= {} */
	)
{
	BoneWeights.Sort(FBoneWeight::DescSortByWeightPredicate);

	int32 NumEntries = FMath::Min(BoneWeights.Num(), InSettings.GetMaxWeightCount());
	if (NumEntries == 0 && InSettings.HasDefaultBoneIndex())
	{
		ContainerAdapter::SetNum(Container, 1);
		ContainerAdapter::Set(Container, 0, FBoneWeight(InSettings.GetDefaultBoneIndex(), FBoneWeight::GetMaxRawWeight()));
		return;
	}

	ContainerAdapter::SetNum(Container, NumEntries);

	for (int32 Index = 0; Index < NumEntries; Index++)
	{
		ContainerAdapter::Set(Container, Index, BoneWeights[Index]);
	}
	NormalizeWeights(InSettings.GetNormalizeType());
}

template<typename ContainerAdapter>
template<typename CT>
typename std::enable_if<!std::is_const<CT>::value, bool>::type
TBoneWeights<ContainerAdapter>::AddBoneWeight(
	FBoneWeight InBoneWeight,
	const FBoneWeightsSettings& InSettings /*= {}*/)
{
	// Does this bone already exist?
	int32 WeightIndex = FindWeightIndexByBone(InBoneWeight.GetBoneIndex());

	// If the sum of weights could possibly exceed 1.0, we may need normalization based on
	// the weight settings.
	bool bMayNeedNormalization;

	if (WeightIndex != INDEX_NONE)
	{
		FBoneWeight ExistingBoneWeight = ContainerAdapter::Get(Container, WeightIndex);

		// New weight is below the threshold. Remove the current bone weight altogether.
		if (InBoneWeight.GetRawWeight() < InSettings.GetRawWeightThreshold())
		{
			ContainerAdapter::Remove(Container, WeightIndex);

			// If always normalizing, we need to re-normalize after removing this entry.
			if (InSettings.GetNormalizeType() == EBoneWeightNormalizeType::Always)
			{
				NormalizeWeights(EBoneWeightNormalizeType::Always);
			}

			return false;
		}

		if (ExistingBoneWeight.GetRawWeight() == InBoneWeight.GetRawWeight())
		{
			return true;
		}

		bMayNeedNormalization = (ExistingBoneWeight.GetRawWeight() < InBoneWeight.GetRawWeight());

		ExistingBoneWeight.SetRawWeight(InBoneWeight.GetRawWeight());

		ContainerAdapter::Set(Container, WeightIndex, ExistingBoneWeight);
	}
	else
	{
		// If the new weight is below the threshold, reject and return.
		if (InBoneWeight.GetRawWeight() < InSettings.GetRawWeightThreshold())
		{
			return false;
		}

		// Are we already at the limit of weights for this container?
		const int32 NumWeights = ContainerAdapter::Num(Container);
		if (NumWeights == InSettings.GetMaxWeightCount())
		{
			// If the weight is smaller than the smallest weight currently, then we reject.
			if (InBoneWeight.GetRawWeight() < ContainerAdapter::Get(Container, NumWeights - 1).GetRawWeight())
			{
				return false;
			}

			// Overwrite the last one, we'll put it in its correct place when we sort.
			ContainerAdapter::Set(Container, NumWeights - 1, InBoneWeight);
		}
		else
		{
			ContainerAdapter::Add(Container, InBoneWeight);
		}

		bMayNeedNormalization = true;
	}

	// If we got here, then we updated/added weights. We're contractually obligated to keep the
	// weights sorted.
	SortWeights();

	if ((InSettings.GetNormalizeType() == EBoneWeightNormalizeType::Always) ||
		(InSettings.GetNormalizeType() == EBoneWeightNormalizeType::AboveOne && bMayNeedNormalization))
	{
		Renormalize(InSettings);
	}

	return true;
}


template<typename ContainerAdapter>
template<typename CT>
typename std::enable_if<!std::is_const<CT>::value, bool>::type
TBoneWeights<ContainerAdapter>::RemoveBoneWeight(
    FBoneIndexType InBoneIndex,
	const FBoneWeightsSettings& InSettings /*= {}*/
	)
{
	int32 WeightIndex = FindWeightIndexByBone(InBoneIndex);
	if (WeightIndex == INDEX_NONE)
	{
		return false;
	}

	ContainerAdapter::Remove(Container, WeightIndex);

	// Cull all weights that exceed limits set by the settings.
	CullWeights(InSettings);

	// Removing weights will always cause the weight sum to decrease, so we only have to normalize
	// if always asked to.
	if (InSettings.GetNormalizeType() == EBoneWeightNormalizeType::Always)
	{
		NormalizeWeights(EBoneWeightNormalizeType::Always);
	}

	return true;
}


template<typename ContainerAdapter>
template<typename CT>
typename std::enable_if<!std::is_const<CT>::value, void>::type
TBoneWeights<ContainerAdapter>::Renormalize(
	const FBoneWeightsSettings& InSettings /*= {}*/
	)
{
	NormalizeWeights(InSettings.GetNormalizeType());

	// If entries are now below the threshold, remove them.
	if (InSettings.GetNormalizeType() == EBoneWeightNormalizeType::Always && CullWeights(InSettings))
	{
		NormalizeWeights(EBoneWeightNormalizeType::Always);
	}
}


template<typename ContainerAdapter>
template<typename ContainerAdapterA, typename ContainerAdapterB, typename CT>
typename std::enable_if<!std::is_const<CT>::value, void>::type
TBoneWeights<ContainerAdapter>::Blend(
	const TBoneWeights<ContainerAdapterA>& InBoneWeightsA,
	const TBoneWeights<ContainerAdapterB>& InBoneWeightsB,
	const float InBias,
	const FBoneWeightsSettings& InSettings /*= {}*/
	)
{
	checkSlow(InBoneWeightsA.Verify());
	checkSlow(InBoneWeightsB.Verify());

	// Both empty?
	if (InBoneWeightsA.Num() == 0 && InBoneWeightsB.Num() == 0)
	{
		if (InSettings.HasDefaultBoneIndex())
		{
			ContainerAdapter::SetNum(Container, 1);
			ContainerAdapter::Set(Container, 0, FBoneWeight{InSettings.GetDefaultBoneIndex(), FBoneWeight::GetMaxRawWeight()});
		}
		else
		{
			ContainerAdapter::SetNum(Container, 0);
		}		
		return;
	}
	// FIXME: We can probably special-case a few more fast paths (one on either side, one each). 
	// But let's collect statistics first.

	// To simplify lookup and iteration over the two bone weight arrays, we sort by bone index
	// value, but indirectly, since we can't sort them directly, as that would violate the
	// sorted-by-descending-weight contract. Instead we create an indirection array on the stack
	// and use that to iterate
	auto CreateIndirectIndex = [](const auto &InBoneWeights, TArrayView<int32> InIndexIndirect) {
		for (int32 Index = 0; Index < InIndexIndirect.Num(); Index++)
		{
			InIndexIndirect[Index] = Index;
		}
		InIndexIndirect.Sort([InBoneWeights](int32 A, int32 B) {
			return InBoneWeights[A].GetBoneIndex() < InBoneWeights[B].GetBoneIndex();
		});
	};

	TArray<int32, TInlineAllocator<MaxInlineBoneWeightCount>> IndirectIndexA;
	IndirectIndexA.SetNumUninitialized(InBoneWeightsA.Num());
	CreateIndirectIndex(InBoneWeightsA, IndirectIndexA);

	TArray<int32, TInlineAllocator<MaxInlineBoneWeightCount>> IndirectIndexB;
	IndirectIndexB.SetNumUninitialized(InBoneWeightsB.Num());
	CreateIndirectIndex(InBoneWeightsB, IndirectIndexB);

	TArray<FBoneWeight, TInlineAllocator<MaxInlineBoneWeightCount * 2>> BoneWeights;
	BoneWeights.Reserve(InBoneWeightsA.Num() + InBoneWeightsB.Num());

	const int32 RawBiasB = static_cast<int32>(InBias * static_cast<float>(FBoneWeight::GetMaxRawWeight()));
	const int32 RawBiasA = FBoneWeight::GetMaxRawWeight() - RawBiasB;

	auto BlendWithZeroInfluenceBone = [&InSettings, &BoneWeights](const FBoneWeight& BW, const int32 RawBias) 
	{
		if (InSettings.GetBlendZeroInfluence())  
		{
			// Treat the missing bone as having a zero weight 
			uint16 RawWeight = uint16( (int32)BW.GetRawWeight() * RawBias / FBoneWeight::GetMaxRawWeight() );
			BoneWeights.Emplace(BW.GetBoneIndex(), RawWeight);
		}
		else 
		{
			// Ignore the missing bone
			BoneWeights.Add(BW);
		}	
	}; 

	int32 IndexA = 0, IndexB = 0;
	for (; IndexA < InBoneWeightsA.Num() && IndexB < InBoneWeightsB.Num(); /* */ )
	{
		const FBoneWeight& BWA = InBoneWeightsA[IndirectIndexA[IndexA]];
		const FBoneWeight& BWB = InBoneWeightsB[IndirectIndexB[IndexB]];

		// If both have the same bone index, we blend them using the bias given and advance
		// both arrays. If the bone indices differ, we copy from the array with the lower bone
		// index value, to ensure we can possibly catch up with the other array. We then
		// advance until we hit the end of either array after which we blindly copy the remains.
		if (BWA.GetBoneIndex() == BWB.GetBoneIndex())
		{
			uint16 RawWeight = uint16( ((int32)BWA.GetRawWeight() * RawBiasA + (int32)BWB.GetRawWeight() * RawBiasB) / FBoneWeight::GetMaxRawWeight() );

			BoneWeights.Emplace(BWA.GetBoneIndex(), RawWeight);
			IndexA++;
			IndexB++;
		}
		else if (BWA.GetBoneIndex() < BWB.GetBoneIndex())
		{
			BlendWithZeroInfluenceBone(BWA, RawBiasA);
			IndexA++;
		}
		else
		{
			BlendWithZeroInfluenceBone(BWB, RawBiasB);
			IndexB++;
		}
	}

	for (; IndexA < InBoneWeightsA.Num(); IndexA++)
	{
		BlendWithZeroInfluenceBone(InBoneWeightsA[IndirectIndexA[IndexA]], RawBiasA);
	}
	for (; IndexB < InBoneWeightsB.Num(); IndexB++)
	{
		BlendWithZeroInfluenceBone(InBoneWeightsB[IndirectIndexB[IndexB]], RawBiasB);		
	}

	SetBoneWeightsInternal(BoneWeights, InSettings);
}


template<typename ContainerAdapter>
void TBoneWeights<ContainerAdapter>::SortWeights()
{
	ContainerAdapter::Sort(Container, FBoneWeight::DescSortByWeightPredicate);
}


template<typename ContainerAdapter>
bool TBoneWeights<ContainerAdapter>::CullWeights(
	const FBoneWeightsSettings& InSettings
	)
{
	bool bCulled = false;
	int32 NumWeights = ContainerAdapter::Num(Container);

	// If there are more entries in the container than the settings allow for, indiscriminately 
	// remove the excess entries.
	if (NumWeights > InSettings.GetMaxWeightCount())
	{
		ContainerAdapter::SetNum(Container, InSettings.GetMaxWeightCount());
		NumWeights = InSettings.GetMaxWeightCount();
		bCulled = true;
	}

	// If any remaining entries are now below the threshold, remove them too.
	while (NumWeights > 0 && ContainerAdapter::Get(Container, NumWeights - 1).GetRawWeight() < InSettings.GetRawWeightThreshold())
	{
		ContainerAdapter::SetNum(Container, --NumWeights);
		bCulled = true;
	}

	return bCulled;
}


template<typename ContainerAdapter>
void TBoneWeights<ContainerAdapter>::NormalizeWeights(
	EBoneWeightNormalizeType InNormalizeType
	)
{
	const int32 NumWeights = ContainerAdapter::Num(Container);

	// Early checks
	if (InNormalizeType == EBoneWeightNormalizeType::None || NumWeights == 0)
	{
		return;
	}

	// Common case.
	if (NumWeights == 1)
	{
		if (InNormalizeType == EBoneWeightNormalizeType::Always)
		{
			// Set the weight to full for the sole entry if normalizing always.
			FBoneWeight BoneWeight = ContainerAdapter::Get(Container, 0);
			BoneWeight.SetRawWeight(FBoneWeight::GetMaxRawWeight());
			ContainerAdapter::Set(Container, 0, BoneWeight);
		}
		return;
	}

	// We operate on int64, since we can easily end up with wraparound issues during one of the
	// multiplications below when using int32. This would tank the division by WeightSum.
	int64 WeightSum = 0;
	bool bNeedResort = false;
	uint16 LastWeight = std::numeric_limits<uint16>::max();
	
	for (int32 Index = 0; Index < NumWeights; Index++)
	{
		const uint16 NewWeight = ContainerAdapter::Get(Container, Index).GetRawWeight();
		WeightSum += NewWeight;

		if (NewWeight > LastWeight)
		{
			bNeedResort = true;
		}
		LastWeight = NewWeight;
	}

	// 
	if ((InNormalizeType == EBoneWeightNormalizeType::Always && WeightSum != FBoneWeight::GetMaxRawWeight() && ensure(WeightSum != 0)) ||
		WeightSum > FBoneWeight::GetMaxRawWeight())
	{
		int64 Correction = 0;

		// Here we treat the raw weight as a 16.16 fixed point value and ensure that the
		// fraction, which would otherwise be lost through rounding, is carried over to the
		// subsequent values to maintain a constant sum to the max weight value.
		// We do this in descending weight order in an attempt to ensure that weight values
		// aren't needlessly lost after scaling.
		// It can happen, if all the weights are the same, or similar, that the last weight may
		// now be greater in value than the others. In that case we re-sort to ensure that our
		// descending weight order invariant holds. 
		LastWeight = std::numeric_limits<uint16>::max();
		for (int32 Index = 0; Index < NumWeights; Index++)
		{
			FBoneWeight BW = ContainerAdapter::Get(Container, Index);
			const int64 ScaledWeight = static_cast<int64>(BW.GetRawWeight()) * FBoneWeight::GetMaxRawWeight() + Correction;
			const uint16 NewWeight = static_cast<uint16>(FMath::Min(ScaledWeight / WeightSum, static_cast<int64>(FBoneWeight::GetMaxRawWeight()))); 
			BW.SetRawWeight(NewWeight);
			Correction = ScaledWeight - BW.GetRawWeight() * WeightSum;
			ContainerAdapter::Set(Container, Index, BW);

			// Due to error distribution, we can end up with weights out of order again.
			if (NewWeight > LastWeight)
			{
				bNeedResort = true;
			}
			LastWeight = NewWeight;
		}
	}
	if (bNeedResort)
	{
		SortWeights();
	}
}


template<typename ContainerAdapter>
bool UE::AnimationCore::TBoneWeights<ContainerAdapter>::Verify() const
{
	const int32 NumEntries = Num(); 
	if (NumEntries == 0)
	{
		return true;
	}
	
	// Check that bone indexes are unique
	TSet<FBoneIndexType> BoneIndexes;
	BoneIndexes.Reserve(NumEntries);
	for (int32 Index = 0; Index < NumEntries; Index++)
	{
		FBoneWeight BW = Get(Index);
		if (BoneIndexes.Find(BW.GetBoneIndex()) != nullptr)
		{
			// Commented out for now, to avoid linker errors, since this is a header-only file.
			// UE_LOGF(LogAnimationCore, Error, "Bone Index %d is duplicated", static_cast<int>(BW.GetBoneIndex()));
			return false;
		}
		BoneIndexes.Add(BW.GetBoneIndex());
	}
	
	// Check that all weights are ordered.
	float LastWeight = Get(0).GetWeight();
	for (int32 Index = 1; Index < NumEntries; Index++)
	{
		const float Weight = Get(Index).GetWeight();
		if (Weight > LastWeight)
		{
			//UE_LOGF(LogAnimationCore, Error, "Bone Weight at %d is greater than previous (%g > %g)",
			//	Index, Weight, LastWeight);
			return false;
		}
		LastWeight = Weight;
	}		
	
	return true;
}


// FBoneWeights implementations
static auto WeightSortPredicate = [](const FBoneWeight& A, const FBoneWeight& B) {
	return A.GetRawWeight() > B.GetRawWeight();
};


bool FBoneWeights::SetBoneWeight(
    FBoneWeight InBoneWeight,
    const FBoneWeightsSettings& InSettings /*= {} */
)
{
	FArrayWrapper W(BoneWeights);
	return W.AddBoneWeight(InBoneWeight, InSettings);
}


bool FBoneWeights::RemoveBoneWeight(
    FBoneIndexType InBoneIndex,
    const FBoneWeightsSettings& InSettings /*= {} */
)
{
	FArrayWrapper W(BoneWeights);
	return W.RemoveBoneWeight(InBoneIndex, InSettings);
}


void FBoneWeights::Renormalize(const FBoneWeightsSettings& InSettings /*= {}*/)
{
	FArrayWrapper W(BoneWeights);
	return W.Renormalize(InSettings);
}


FBoneWeights FBoneWeights::Create(
    const FBoneIndexType InBones[MaxInlineBoneWeightCount],
    const uint16 InInfluences[MaxInlineBoneWeightCount],
    const FBoneWeightsSettings& InSettings /*= {} */
)
{
	FBoneWeights Result;
	FArrayWrapper W(Result.BoneWeights);
	W.SetBoneWeights(InBones, InInfluences, InSettings);
	return Result;
}


FBoneWeights FBoneWeights::Create(
    const FBoneIndexType* InBones,
    const float* InInfluences,
    int32 NumEntries,
    const FBoneWeightsSettings& InSettings /*= {} */
)
{
	FBoneWeights Result;
	FArrayWrapper W(Result.BoneWeights);
	W.SetBoneWeights(InBones, InInfluences, NumEntries, InSettings);
	return Result;
}


FBoneWeights FBoneWeights::Create(
	TArrayView<const FBoneWeight> InBoneWeights, 
	const FBoneWeightsSettings& InSettings /*= {} */
)
{
	FBoneWeights Result;
	FArrayWrapper(Result.BoneWeights).SetBoneWeights(InBoneWeights, InSettings);
	return Result;
}


FBoneWeights FBoneWeights::Blend(
    const FBoneWeights& InA,
    const FBoneWeights& InB,
    float InBias,
    const FBoneWeightsSettings& InSettings /*= {} */
)
{
	FBoneWeights Result;
	FArrayWrapper W(Result.BoneWeights);
	FArrayWrapper A(const_cast<FBoneWeights&>(InA).BoneWeights);
	FArrayWrapper B(const_cast<FBoneWeights&>(InB).BoneWeights);
	W.Blend(A, B, InBias, InSettings);
	return Result;
}

FBoneWeights FBoneWeights::Blend(
	const FBoneWeights& InA,
	const FBoneWeights& InB,
	const FBoneWeights& InC,
	float InBaryX,
	float InBaryY,
	float InBaryZ,
	const FBoneWeightsSettings& InSettings) 
{
	if (FMath::IsNearlyZero(InBaryY + InBaryZ) == false)
	{
		const float BCW = InBaryZ / (InBaryY + InBaryZ);
		
		// Use the default settings to make sure we don't normalize, don't prune bones if we exceed the default limit 
		// and treat the bones with zero influence as having a zero weight during the first blend
		FBoneWeightsSettings FirstBlendSettings;
		FirstBlendSettings.SetNormalizeType(UE::AnimationCore::EBoneWeightNormalizeType::None); 
		FirstBlendSettings.SetMaxWeightCount(InB.Num() + InC.Num());
		
		const FBoneWeights BC = FBoneWeights::Blend(InB, InC, BCW, FirstBlendSettings);
		const FBoneWeights BCA = FBoneWeights::Blend(BC, InA, InBaryX, InSettings);
		return BCA;
	}
	else
	{
		return InA;
	}
}


} // namespace AnimationCore
} // namespace UE


/**
* A hashing function to allow the FBoneWeight class to be used with hashing containers (e.g.
* TSet or TMap).
*/
static inline uint32 GetTypeHash(
	const UE::AnimationCore::FBoneWeight& InBoneWeight
	)
{
	return InBoneWeight.GetTypeHash();
}

static inline FArchive& operator<<(
	FArchive& InArchive, 
	UE::AnimationCore::FBoneWeight& InOutBoneWeight
	)
{
	InOutBoneWeight.Serialize(InArchive);
	return InArchive;
}

static inline uint32 GetTypeHash(const UE::AnimationCore::FBoneWeights& InBoneWeights)
{
	return InBoneWeights.GetTypeHash();
}

static inline FArchive& operator<<(
    FArchive& InArchive,
    UE::AnimationCore::FBoneWeights& InOutBoneWeights)
{
	InOutBoneWeights.Serialize(InArchive);
	return InArchive;
}
