// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneIndices.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "Math/Transform.h"
#include "UObject/ObjectMacros.h"

#include "CCDIK.generated.h"

/** Transient structure for CCDIK node evaluation */
USTRUCT()
struct FCCDIKChainLink
{
	GENERATED_USTRUCT_BODY()

public:
	/** Transform of bone in component space. */
	FTransform Transform;

	/** Transform of bone in local space. This is mutable as their component space changes or parents*/
	FTransform LocalTransform;

	/** Transform Index that this control will output */
	int32 TransformIndex;

	/** Child bones which are overlapping this bone. 
	 * They have a zero length distance, so they will inherit this bone's transformation. */
	TArray<int32> ChildZeroLengthTransformIndices;

	double CurrentAngleDelta;

	FCCDIKChainLink()
		: TransformIndex(INDEX_NONE)
		, CurrentAngleDelta(0.0)
	{
	}

	FCCDIKChainLink(const FTransform& InTransform, const FTransform& InLocalTransform, const int32& InTransformIndex)
		: Transform(InTransform)
		, LocalTransform(InLocalTransform)
		, TransformIndex(InTransformIndex)
		, CurrentAngleDelta(0.0)
	{
	}
};

namespace AnimationCore
{
	ANIMATIONCORE_API bool SolveCCDIK(TArray<FCCDIKChainLink>& InOutChain, const FVector& TargetPosition, float Precision, int32 MaxIteration, bool bStartFromTail, bool bEnableRotationLimit, const TArray<float>& RotationLimitPerJoints);
};
