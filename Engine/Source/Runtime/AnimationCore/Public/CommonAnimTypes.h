// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AnimationTypes.h: Render core module definitions.
=============================================================================*/

#pragma once
#include "CoreMinimal.h"
#include "CommonAnimTypes.generated.h"

/** Axis to represent direction */
USTRUCT()
struct FAxis
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "FAxis")
	FVector Axis;

	UPROPERTY(EditAnywhere, Category = "FAxis")
	bool bInLocalSpace;

	FAxis(const FVector& InAxis = FVector::ForwardVector)
		:  bInLocalSpace(true) 
	{
		Axis = InAxis.GetSafeNormal();
	};

	/** return transformed axis based on ComponentSpaceTransform */
	FVector GetTransformedAxis(const FTransform& ComponentSpaceTransform) const
	{
		if (bInLocalSpace)
		{
			return ComponentSpaceTransform.TransformVectorNoScale(Axis);
		}

		// if world transform, we don't have to transform
		return Axis;
	}

	/** Initialize the set up */
	void Initialize()
	{
		Axis = Axis.GetSafeNormal();
	}

	/** return true if Valid data */
	bool IsValid() const
	{
		return Axis.IsNormalized();
	}

	friend FArchive & operator<<(FArchive & Ar, FAxis & D)
	{
		Ar << D.Axis;
		Ar << D.bInLocalSpace;

		return Ar;
	}
};

