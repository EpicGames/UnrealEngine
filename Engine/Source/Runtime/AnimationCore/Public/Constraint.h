// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Constraint.h: Constraint data structures

	This section has and will change a lot while working on our rigging system
	We strongly recommend not to use this directly yet but by provided tool, such 
	as Constraint AnimNode. 
=============================================================================*/

#pragma once

#include "CommonAnimTypes.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EulerTransform.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/TransformVectorized.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealNames.h"

#include "Constraint.generated.h"

struct FMultiTransformBlendHelper;

/**
 * Filter Option Per Axis
 * 
 * This is used to filter per axis for constraint options
 */
USTRUCT(BlueprintType)
struct FFilterOptionPerAxis
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Axis Filter")
	bool bX;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Axis Filter")
	bool bY;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Axis Filter")
	bool bZ;

	FFilterOptionPerAxis()
		: bX(true)
		, bY(true)
		, bZ(true)
	{}

	FFilterOptionPerAxis(bool X, bool Y, bool Z)
		: bX(X)
		, bY(Y)
		, bZ(Z)
	{}

	void FilterVector(FVector& Input, const FVector& ResetValue = FVector::ZeroVector) const
	{
		if (!bX)
		{
			Input.X = ResetValue.X;
		}

		if (!bY)
		{
			Input.Y = ResetValue.Y;
		}

		if (!bZ)
		{
			Input.Z = ResetValue.Z;
		}
	}

	void FilterQuat(FQuat& Input, const FQuat& ResetValue = FQuat::Identity) const
	{
		FRotator Rotator = Input.Rotator();

		FilterRotator(Rotator, ResetValue.Rotator());

		Input = Rotator.Quaternion();
	}

	void FilterRotator(FRotator& Input, const FRotator& ResetValue = FRotator::ZeroRotator) const
	{
		if (!bX)
		{
			Input.Roll = ResetValue.Roll;
		}

		if (!bY)
		{
			Input.Pitch = ResetValue.Pitch;
		}

		if (!bZ)
		{
			Input.Yaw = ResetValue.Yaw;
		}
	}

	friend FArchive & operator<<(FArchive & Ar, FFilterOptionPerAxis & D)
	{
		Ar << D.bX;
		Ar << D.bY;
		Ar << D.bZ;

		return Ar;
	}

	bool IsValid() const
	{
		// if none of them is set, it's not valid
		return bX || bY || bZ;
	}
	
	bool HasNoEffect() const
	{
		// if all of them are set the filter won't affect anything
		return bX && bY && bZ;
	}
};

/** A filter for a whole transform */
USTRUCT(BlueprintType)
struct FTransformFilter
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Axis Filter")
	FFilterOptionPerAxis TranslationFilter;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Axis Filter")
	FFilterOptionPerAxis RotationFilter;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Axis Filter")
	FFilterOptionPerAxis ScaleFilter;

	void FilterTransform(FTransform& Input) const
	{
		FVector Location = Input.GetLocation();
		TranslationFilter.FilterVector(Location);
		Input.SetLocation(Location);

		FQuat Rotation = Input.GetRotation();
		RotationFilter.FilterQuat(Rotation);
		Input.SetRotation(Rotation);

		FVector Scale3D = Input.GetScale3D();
		ScaleFilter.FilterVector(Scale3D, FVector::OneVector);
		Input.SetScale3D(Scale3D);
	}

	void FilterTransform(FEulerTransform& Input) const
	{
		FVector Location = Input.Location;
		TranslationFilter.FilterVector(Location);
		Input.Location = Location;

		FRotator Rotation = Input.Rotation;
		RotationFilter.FilterRotator(Rotation);
		Input.Rotation = Rotation;

		FVector Scale = Input.Scale;
		ScaleFilter.FilterVector(Scale, FVector::OneVector);
		Input.Scale = Scale;
	}
};

/** A description of how to apply a simple transform constraint */
USTRUCT(BlueprintType)
struct FConstraintDescription
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Constraint")
	bool bTranslation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Constraint")
	bool bRotation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Constraint")
	bool bScale;

	// this does composed transform - where as individual will accumulate per component
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Constraint")
	bool bParent;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Constraint")
	FFilterOptionPerAxis TranslationAxes;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Constraint")
	FFilterOptionPerAxis RotationAxes;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Constraint")
	FFilterOptionPerAxis ScaleAxes;

	FConstraintDescription()
		: bTranslation(true)
		, bRotation(true)
		, bScale(false)
		, bParent(false)
	{
	}

	friend FArchive & operator<<(FArchive & Ar, FConstraintDescription & D)
	{
		Ar << D.bTranslation;
		Ar << D.bRotation;
		Ar << D.bScale;
		Ar << D.bParent;
		Ar << D.TranslationAxes;
		Ar << D.RotationAxes;
		Ar << D.ScaleAxes;

		return Ar;
	}
};

/**
 * This is the offset for constraint
 *
 * Saves individual component (translation, rotation, scale or parent)
 * Used by Constraint for saving the offset, and recovering the offset
 */
USTRUCT()
struct FConstraintOffset
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FVector Translation;

	UPROPERTY()
	FQuat Rotation;

	UPROPERTY()
	FVector Scale;

	UPROPERTY()
	FTransform Parent;

	FConstraintOffset()
		: Translation(FVector::ZeroVector)
		, Rotation(FQuat::Identity)
		, Scale(FVector::OneVector)
		, Parent(FTransform::Identity)
	{}

	/* Apply the Inverse offset */
	ANIMATIONCORE_API void ApplyInverseOffset(const FTransform& InTarget, FTransform& OutSource) const;
	/* Save the Inverse offset */
	ANIMATIONCORE_API void SaveInverseOffset(const FTransform& Source, const FTransform& Target, const FConstraintDescription& Operator);
	/** Clear the offset */
	void Reset()
	{
		Translation = FVector::ZeroVector;
		Rotation = FQuat::Identity;
		Scale = FVector::OneVector;
		Parent = FTransform::Identity;
	}

	friend FArchive & operator<<(FArchive & Ar, FConstraintOffset & D)
	{
		Ar << D.Translation;
		Ar << D.Rotation;
		Ar << D.Scale;
		Ar << D.Parent;

		return Ar;
	}
};

USTRUCT(BlueprintType)
struct FTransformConstraint
{
	GENERATED_USTRUCT_BODY()

	// @note thought of separating this out per each but we'll have an issue with applying transform in what order
	// but something to think about if that seems better
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Transform Constraint")
	FConstraintDescription Operator;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Transform Constraint")
	FName SourceNode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Transform Constraint")
	FName TargetNode;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Transform Constraint")
	float Weight;

	/** When the constraint is first applied, maintain the offset from the target node */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Transform Constraint")
	bool bMaintainOffset;

	FTransformConstraint()
		: SourceNode(NAME_None)
		, TargetNode(NAME_None)
		, Weight(1.f)
		, bMaintainOffset(true)
	{}

	friend FArchive & operator<<(FArchive & Ar, FTransformConstraint & D)
	{
		Ar << D.Operator;
		Ar << D.SourceNode;
		Ar << D.TargetNode;
		Ar << D.Weight;
		Ar << D.bMaintainOffset;

		return Ar;
	}
};

////////////////////////////////////////////////////////////////
/// new changes of constraints

/** Constraint Types*/
UENUM(BlueprintType)
enum class EConstraintType : uint8
{
	/** Transform Constraint */
	Transform,

	/** Aim Constraint*/
	Aim,

	/** MAX - invalid */
	MAX,
};

/** A description of how to apply a simple transform constraint */
USTRUCT()
struct FConstraintDescriptionEx
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = FAimConstraintDescription)
	FFilterOptionPerAxis AxesFilterOption;

	virtual ~FConstraintDescriptionEx() = default;

	/**
	 * Apply Constraint : Apply Constraint transform to BlendHelperInLocalSpace in local space
	 * 
	 * @param Target Transform: Current Target Transform in global space
	 * @param Current Transform: Current Source Transform in global space
	 * @param Current Parent Transform: Current Source Parent Transform in global space
	 * @param Weight : Current Weight
	 * @param BlendHelperInLocalSpace : Blend Helper, this accumulates all constraints transform and later on blend to final transform
	 *
	 * @return BlendHelperInLocalSpace will contains constraint's local transform result, it is local because that's how you want to compose multiple to one transform at the end
	 */
	virtual void AccumulateConstraintTransform(const FTransform& TargetTransform, const FTransform& CurrentTransform, const FTransform& CurrentParentTransform, float Weight, FMultiTransformBlendHelper& BlendHelperInLocalSpace) const PURE_VIRTUAL(AccumulateConstraintTransform, );

	/** 
	 * Functions that describes what they modify
	 *
	 * Since same component will be blended by weight correctly, this has to split to each component
	 */
	virtual bool DoesAffectRotation() const { return false;  }
	virtual bool DoesAffectTranslation() const { return false; }
	virtual bool DoesAffectScale() const { return false; }
	/**
	* Functions that describes what they modify - this means, whole Transform, so combined transform, not individual component
	* This will override any individual component if returning true
	*/
	virtual bool DoesAffectTransform() const { return false; }

	virtual FString GetDisplayString() const PURE_VIRTUAL(GetDisplayString, return TEXT("None"););

	/** 
	 * Serializer 
	 */
	virtual void Serialize(FArchive& Ar)
	{
		Ar << AxesFilterOption;
	}

	friend FArchive & operator<<(FArchive & Ar, FConstraintDescriptionEx & D)
	{
		D.Serialize(Ar);
		return Ar;
	}
};
template<>
struct TStructOpsTypeTraits<FConstraintDescriptionEx> : public TStructOpsTypeTraitsBase2<FConstraintDescriptionEx>
{
	enum
	{
		WithPureVirtual = true,
	};
};

/** Transform Constraint Types*/
UENUM(Blueprintable)
enum class ETransformConstraintType : uint8
{
	Translation,
	Rotation,
	Scale, 
	Parent,
	LookAt
};

/** A description of how to apply a simple transform constraint */
USTRUCT()
struct FTransformConstraintDescription : public FConstraintDescriptionEx
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = FAimConstraintDescription)
	ETransformConstraintType TransformType;

	FTransformConstraintDescription(const ETransformConstraintType InType = ETransformConstraintType::Translation)
		:TransformType (InType)
	{}
	
	ANIMATIONCORE_API virtual void AccumulateConstraintTransform(const FTransform& TargetTransform, const FTransform& CurrentTransform, const FTransform& CurrentParentTransform, float Weight, FMultiTransformBlendHelper& BlendHelperInLocalSpace) const override;
	virtual bool DoesAffectRotation() const override { return TransformType == ETransformConstraintType::Rotation; }
	virtual bool DoesAffectTranslation() const override { return TransformType == ETransformConstraintType::Translation; }
	virtual bool DoesAffectScale() const override { return TransformType == ETransformConstraintType::Scale; }
	virtual bool DoesAffectTransform() const override { return TransformType == ETransformConstraintType::Parent; }

	virtual FString GetDisplayString() const override
	{
		switch (TransformType)
		{
		case ETransformConstraintType::Parent:
			return TEXT("Parent");
		case ETransformConstraintType::Translation:
			return TEXT("Translation");
		case ETransformConstraintType::Rotation:
			return TEXT("Rotation");
		case ETransformConstraintType::Scale:
			return TEXT("Scale");
		default:
			ensure(false);
		}

		return TEXT("None");
	}

	virtual void Serialize(FArchive& Ar) override
	{
		FConstraintDescriptionEx::Serialize(Ar);
		Ar << TransformType;
	}
};

/** A description of how to apply aim constraint */
USTRUCT()
struct FAimConstraintDescription : public FConstraintDescriptionEx
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = FAimConstraintDescription)
	FAxis LookAt_Axis;
	UPROPERTY(EditAnywhere, Category = FAimConstraintDescription)
	FAxis LookUp_Axis;
	UPROPERTY(EditAnywhere, Category = FAimConstraintDescription)
	bool bUseLookUp;
	UPROPERTY(EditAnywhere, Category = FAimConstraintDescription)
	FVector LookUpTarget;

	FAimConstraintDescription()
		: LookAt_Axis()
		, LookUp_Axis(FVector::UpVector)
		, bUseLookUp(false)
		, LookUpTarget(ForceInitToZero)
	{
	}

	ANIMATIONCORE_API virtual void AccumulateConstraintTransform(const FTransform& TargetTransform, const FTransform& CurrentTransform, const FTransform& CurrentParentTransform, float Weight, FMultiTransformBlendHelper& BlendHelperInLocalSpace) const override;
	virtual bool DoesAffectRotation() const override { return true; }
	virtual FString GetDisplayString() const override
	{
		return TEXT("Aim");
	}

	virtual void Serialize(FArchive& Ar) override
	{
		FConstraintDescriptionEx::Serialize(Ar);

		Ar << LookAt_Axis;
		Ar << LookUp_Axis;
		Ar << bUseLookUp;
	}
};

/** 
 * Constraint data container. It contains union of Constraints and node will contain array of these. 
 * 
 * These are the one contained in NodeData, and it will be iterated via evaluate process
 * The goal is to have contiguous memory location where they can iterate through linearly
 */
USTRUCT()
struct FConstraintDescriptor
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	EConstraintType Type;

	FConstraintDescriptionEx* ConstraintDescription;

	FConstraintDescriptor()
		: Type(EConstraintType::MAX)
		, ConstraintDescription(nullptr)
	{
	}

	FConstraintDescriptor(const FTransformConstraintDescription& InT)
		: Type(EConstraintType::Transform)
		, ConstraintDescription(nullptr)
	{
		Set(InT);
	}

	FConstraintDescriptor(const FAimConstraintDescription& InA)
		: Type(EConstraintType::Aim)
		, ConstraintDescription(nullptr)
	{
		Set(InA);
	}
	
	FConstraintDescriptor(const FConstraintDescriptor& InOther)
		: ConstraintDescription(nullptr)
	{
		*this = InOther;
	}

	FString GetDisplayString() const
	{
		if (ConstraintDescription)
		{
			return ConstraintDescription->GetDisplayString();
		}

		return TEXT("Null");
	}

	FConstraintDescriptor& operator=(const FConstraintDescriptor& Other)
	{
		this->Clear();

		this->Type = Other.Type;
		if (Other.IsValid())
		{
			if (Other.Type == EConstraintType::Transform)
			{
				this->Set(*Other.GetTypedConstraint<FTransformConstraintDescription>());
			}
			else if (Other.Type == EConstraintType::Aim)
			{
				this->Set(*Other.GetTypedConstraint<FAimConstraintDescription>());
			}
		}

		return *this;
	}

private:
	void Set(const FAimConstraintDescription& InA)
	{
		Clear();
		ConstraintDescription = new FAimConstraintDescription(InA);
	}

	void Set(const FTransformConstraintDescription& InT)
	{
		Clear();
		ConstraintDescription = new FTransformConstraintDescription(InT);
	}

	void Clear()
	{
		if (ConstraintDescription)
		{
			delete ConstraintDescription;
			ConstraintDescription = nullptr;
		}
	}

public:
	// this does not check type - we can, but that is hard to maintain, maybe I'll change later 
	template <typename T>	
	T* GetTypedConstraint() const
	{
		return static_cast<T*>(ConstraintDescription);
	}

	~FConstraintDescriptor()
	{
		Clear();
	}
	friend FArchive & operator<<(FArchive & Ar, FConstraintDescriptor & D)
	{
		Ar << D.Type;

		if (D.Type == EConstraintType::Transform)
		{
			FTransformConstraintDescription Trans;
			Ar << Trans;
			D.Set(Trans);
		}
		else if (D.Type == EConstraintType::Aim)
		{
			FAimConstraintDescription Aim;
			Ar << Aim;
			D.Set(Aim);
		}
		else
		{
			ensure(false);
		}

		return Ar;
	}

	bool IsValid() const 
	{
		return (ConstraintDescription != nullptr);
	}
	bool DoesAffectRotation() const
	{
		if (ConstraintDescription)
		{
			return ConstraintDescription->DoesAffectRotation();
		}
		return false;
	}

	bool DoesAffectTranslation() const
	{
		if (ConstraintDescription)
		{
			return ConstraintDescription->DoesAffectTranslation();
		}
		return false;
	}

	bool DoesAffectScale() const
	{
		if (ConstraintDescription)
		{
			return ConstraintDescription->DoesAffectScale();
		}
		return false;
	}

	bool DoesAffectTransform() const
	{
		if (ConstraintDescription)
		{
			return ConstraintDescription->DoesAffectTransform();
		}
		return false;
	}

	void ApplyConstraintTransform(const FTransform& TargetTransform, const FTransform& CurrentTransform, const FTransform& CurrentParentTransform, float Weight, FMultiTransformBlendHelper& BlendHelperInLocalSpace) const
	{
		if (ConstraintDescription)
		{
			return ConstraintDescription->AccumulateConstraintTransform(TargetTransform, CurrentTransform, CurrentParentTransform, Weight, BlendHelperInLocalSpace);
		}
	}
};

/** 
 * Constraint Data that is contained in Node Datat
 * You can have as many of these per node
 */
USTRUCT()
struct FConstraintData
{
	GENERATED_USTRUCT_BODY()

	/** Constraint Description */
	UPROPERTY()
	FConstraintDescriptor Constraint;
	/** Weight of the constraint */
	UPROPERTY()
	float Weight;
	/** When the constraint is first applied, maintain the offset from the target node */
	UPROPERTY()
	bool bMaintainOffset;
	/** Constraint offset if bMaintainOffset is used */
	UPROPERTY()
	FTransform Offset;

	UPROPERTY(transient)
	FTransform CurrentTransform;

	FConstraintData()
		: Weight(1.f)
		, bMaintainOffset(true)
		, Offset(FTransform::Identity)
	{}

	FConstraintData(const FTransformConstraintDescription& InTrans, FName InTargetNode = NAME_None, float InWeight = 1.f, bool bInMaintainOffset = true, const FTransform& InOffset = FTransform::Identity)
		: Constraint(InTrans)
		, Weight(InWeight)
		, bMaintainOffset(bInMaintainOffset)
		, Offset(InOffset)
	{}

	FConstraintData(const FAimConstraintDescription& InAim, FName InTargetNode = NAME_None, float InWeight = 1.f, bool bInMaintainOffset = true, const FTransform& InOffset = FTransform::Identity)
		: Constraint(InAim)
		, Weight(InWeight)
		, bMaintainOffset(bInMaintainOffset)
		, Offset(InOffset)
	{}

	friend FArchive & operator<<(FArchive & Ar, FConstraintData & D)
	{
		Ar << D.Constraint;
		Ar << D.Weight;
		Ar << D.bMaintainOffset;
		Ar << D.Offset;

		return Ar;
	}

	ANIMATIONCORE_API void ApplyInverseOffset(const FTransform& InTarget, FTransform& OutSource, const FTransform& InBaseTransform) const;
	ANIMATIONCORE_API void SaveInverseOffset(const FTransform& Source, const FTransform& Target, const FTransform& InBaseTransform);
	void ResetOffset()
	{
		Offset = FTransform::Identity;
	}

	ANIMATIONCORE_API void ApplyConstraintTransform(const FTransform& TargetTransform, const FTransform& InCurrentTransform, const FTransform& CurrentParentTransform, FMultiTransformBlendHelper& BlendHelperInLocalSpace) const;
};
