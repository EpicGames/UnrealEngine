// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "UObject/ObjectMacros.h"

#include "EulerTransform.generated.h"

class UScriptStruct;
template <class T> struct TBaseStructure;

UENUM(BlueprintType)
enum class EEulerRotationOrder : uint8
{
	XYZ,
	XZY,
	YXZ,
	YZX,
	ZXY,
	ZYX
};

USTRUCT(BlueprintType)
struct FEulerTransform
{
	GENERATED_BODY()

	typedef FVector::FReal FReal;

	/**
	 * The identity transformation (Rotation = FRotator::ZeroRotator, Translation = FVector::ZeroVector, Scale = (1,1,1)).
	 */
	static ANIMATIONCORE_API const FEulerTransform Identity;

	inline FEulerTransform()
		: Location(ForceInitToZero)
		, Rotation(ForceInitToZero)
		, Scale(FVector::OneVector)
	{
	}

	inline FEulerTransform(const FVector& InLocation, const FRotator& InRotation, const FVector& InScale)
		: Location(InLocation)
		, Rotation(InRotation)
		, Scale(InScale)
	{
	}

	inline FEulerTransform(const FRotator& InRotation, const FVector& InLocation, const FVector& InScale)
		: Location(InLocation)
		, Rotation(InRotation)
		, Scale(InScale)
	{
	}

	inline explicit FEulerTransform(const FTransform& InTransform)
		: Location(InTransform.GetLocation())
		, Rotation(InTransform.GetRotation().Rotator())
		, Scale(InTransform.GetScale3D())
	{

	}

	/** The translation of this transform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Transform")
	FVector Location;

	/** The rotation of this transform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Transform")
	FRotator Rotation;

	/** The scale of this transform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Transform")
	FVector Scale;

	/** Convert to an FTransform */
	inline FTransform ToFTransform() const
	{
		return FTransform(Rotation.Quaternion(), Location, Scale);
	}

	/** Convert from an FTransform */
	inline void FromFTransform(const FTransform& InTransform)
	{
		Location = InTransform.GetLocation();
		Rotation = InTransform.GetRotation().Rotator();
		Scale = InTransform.GetScale3D();
	}

	// Test if all components of the transforms are equal, within a tolerance.
	inline bool Equals(const FEulerTransform& Other, FReal Tolerance = KINDA_SMALL_NUMBER) const
	{
		return Location.Equals(Other.Location, Tolerance) &&
			Rotation.Equals(Other.Rotation, Tolerance) &&
			Scale.Equals(Other.Scale, Tolerance);
	}

	inline const FVector& GetLocation() const { return Location; }
	inline FQuat GetRotation() const { return Rotation.Quaternion(); }
	inline const FRotator& Rotator() const { return Rotation; }
	inline const FVector& GetScale3D() const { return Scale; }
	inline void SetLocation(const FVector& InValue) { Location = InValue; }
	inline void SetRotation(const FQuat& InValue) { Rotation = InValue.Rotator(); }
	inline void SetRotator(const FRotator& InValue) { Rotation = InValue; }
	inline void SetScale3D(const FVector& InValue) { Scale = InValue; }
	inline void NormalizeRotation() {}
};

template<> struct TBaseStructure<FEulerTransform>
{
	static UScriptStruct* Get() { return FEulerTransform::StaticStruct(); }
};
