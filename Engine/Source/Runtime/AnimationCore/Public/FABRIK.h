// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneIndices.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "UObject/ObjectMacros.h"

#include "FABRIK.generated.h"

struct FBoneContainer;
/**
*	Controller which implements the FABRIK IK approximation algorithm -  see http://andreasaristidou.com/publications/FABRIK.pdf for details
*/

/** Transient structure for FABRIK node evaluation */

/** Transient structure for FABRIK node evaluation */
USTRUCT()
struct FFABRIKChainLink
{
	GENERATED_USTRUCT_BODY()

public:
	/** Position of bone in component space. */
	FVector Position;

	/** Distance to its parent link. */
	double Length;

	/** Bone Index in SkeletalMesh */
	int32 BoneIndex;

	/** Transform Index that this control will output */
	int32 TransformIndex;

	/** Default Direction to Parent */
	FVector DefaultDirToParent;

	/** Child bones which are overlapping this bone.
	* They have a zero length distance, so they will inherit this bone's transformation. */
	TArray<int32> ChildZeroLengthTransformIndices;

	FFABRIKChainLink()
		: Position(FVector::ZeroVector)
		, Length(0.0)
		, BoneIndex(INDEX_NONE)
		, TransformIndex(INDEX_NONE)
		, DefaultDirToParent(FVector(-1.0, 0.0, 0.0))
	{
	}

	FFABRIKChainLink(const FVector& InPosition, const double InLength, const FCompactPoseBoneIndex& InBoneIndex, const int32& InTransformIndex)
		: Position(InPosition)
		, Length(InLength)
		, BoneIndex(InBoneIndex.GetInt())
		, TransformIndex(InTransformIndex)
		, DefaultDirToParent(FVector(-1.0, 0.0, 0.0))
	{
	}

	FFABRIKChainLink(const FVector& InPosition, const double InLength, const FCompactPoseBoneIndex& InBoneIndex, const int32& InTransformIndex, const FVector& InDefaultDirToParent)
		: Position(InPosition)
		, Length(InLength)
		, BoneIndex(InBoneIndex.GetInt())
		, TransformIndex(InTransformIndex)
		, DefaultDirToParent(InDefaultDirToParent)
	{
	}

	FFABRIKChainLink(const FVector& InPosition, const double InLength, const int32 InBoneIndex, const int32 InTransformIndex)
		: Position(InPosition)
		, Length(InLength)
		, BoneIndex(InBoneIndex)
		, TransformIndex(InTransformIndex)
		, DefaultDirToParent(FVector(-1.0, 0.0, 0.0))
	{
	}
};

namespace AnimationCore
{
	/**
	* Fabrik solver
	*
	* This solves FABRIK
	*
	* @param	Chain				Array of chain data
	* @param	TargetPosition		Target for the IK
	* @param	MaximumReach		Maximum Reach
	* @param	Precision			Precision
	* @param	MaxIteration		Number of Max Iteration
	*
	* @return  true if modified. False if not.
	*/
	ANIMATIONCORE_API bool SolveFabrik(TArray<FFABRIKChainLink>& InOutChain, const FVector& TargetPosition, double MaximumReach, double Precision, int32 MaxIteration);
};
