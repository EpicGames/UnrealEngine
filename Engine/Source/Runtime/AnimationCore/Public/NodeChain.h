// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NodeChain.generated.h"

/** A chain of nodes in a hierarchy */
USTRUCT()
struct FNodeChain
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FName> Nodes;
};
