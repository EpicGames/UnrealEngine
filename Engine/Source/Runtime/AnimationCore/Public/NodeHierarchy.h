// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Constraint.h"
#include "Algo/Transform.h"
#include "NodeHierarchy.generated.h"

/** A node in a named hierarchy */
USTRUCT()
struct FNodeObject
{
	GENERATED_BODY()

	FNodeObject()
		: Name(NAME_None)
		, ParentName(NAME_None)
	{}

	FNodeObject(const FName& InName, const FName& InParentName)
		: Name(InName)
		, ParentName(InParentName)
	{}

	/** The name of this node */
	UPROPERTY()
	FName Name;

	/** The name of this node's parent */
	UPROPERTY()
	FName ParentName;
};

/** Hierarchy of nodes */
USTRUCT()
struct FNodeHierarchyData
{
	GENERATED_BODY()

	/** Node hierarchy data */
	UPROPERTY()
	TArray<FNodeObject> Nodes;

	/** Node transform data */
	UPROPERTY()
	TArray<FTransform> Transforms;

	/** Transient look up mapping from name to index to array */
	UPROPERTY()
	TMap<FName, int32> NodeNameToIndexMapping;

public:
	const FTransform& GetTransform(int32 Index) const
	{
		return Transforms[Index];
	}

	FTransform& GetTransform(int32 Index)
	{
		return Transforms[Index];
	}

	void SetTransform(int32 Index, const FTransform& NewTransform)
	{
		Transforms[Index] = NewTransform;
		Transforms[Index].NormalizeRotation();
	}

	int32 GetParentIndex(int32 Index) const
	{
		return GetNodeIndex(Nodes[Index].ParentName);
	}

	FName GetParentName(int32 Index) const
	{
		return Nodes[Index].ParentName;
	}

	void SetParentName(int32 Index, FName NewParent) 
	{
		Nodes[Index].ParentName = NewParent;
	}

	int32 GetNodeIndex(const FName& NodeName) const
	{
		const int32* NodeIndex = NodeNameToIndexMapping.Find(NodeName);
		return (NodeIndex)? *NodeIndex : INDEX_NONE;
	}

	FName GetNodeName(int32 Index) const
	{
		return Nodes[Index].Name;
	}

	void SetNodeName(int32 Index, const FName& NewNodeName)
	{
		FName OldName = Nodes[Index].Name;
		Nodes[Index].Name = NewNodeName;

		// now find all the nodes with this as parent
		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
		{
			if (Nodes[NodeIndex].ParentName == OldName)
			{
				Nodes[NodeIndex].ParentName = NewNodeName;
			}
		}

		BuildNodeNameToIndexMapping();
	}

	int32 Add(const FName& InNodeName, const FName& InParentName, const FTransform& InTransform)
	{
		// already exists
		if (NodeNameToIndexMapping.Contains(InNodeName))
		{
			return INDEX_NONE;
		}

		// if parent name is set, but we don't have it yet?
		if (InParentName != NAME_None && !NodeNameToIndexMapping.Contains(InParentName))
		{
			// warn user?
		}

		check(Nodes.Num() == Transforms.Num());

		int32 NewIndex = Nodes.Add(FNodeObject(InNodeName, InParentName));
		Transforms.Add(InTransform);

		BuildNodeNameToIndexMapping();

		return NewIndex;
	}

	void Empty(int32 Size = 0)
	{
		Nodes.Empty(Size);
		Transforms.Empty(Size);
		NodeNameToIndexMapping.Reset();
	}

	void Allocate(int32 Size)
	{
		Empty(Size);
	}

	/** Returns number of bones in Skeleton. */
	int32 Num() const
	{
		return Nodes.Num();
	}

	bool IsValidIndex(int32 Index) const
	{
		return Nodes.IsValidIndex(Index);
	}

	int32 Remove(const FName& InNodeName)
	{
		int32* NodeIndex = NodeNameToIndexMapping.Find(InNodeName);
		int32 ReturnIndex = INDEX_NONE;
		if (NodeIndex)
		{
			check(Nodes.Num() == Transforms.Num());

			Nodes.RemoveAt(*NodeIndex);
			Transforms.RemoveAt(*NodeIndex);
			ReturnIndex = *NodeIndex;
		}
		else
		{
			// does not exists
		}

		BuildNodeNameToIndexMapping();

		return ReturnIndex;
	}

	void BuildNodeNameToIndexMapping()
	{
		NodeNameToIndexMapping.Empty();

		for (int32 NodeIndex = 0; NodeIndex < Nodes.Num(); ++NodeIndex)
		{
			int32& Index = NodeNameToIndexMapping.Add(Nodes[NodeIndex].Name);
			Index = NodeIndex;
		}

		check(Nodes.Num() == NodeNameToIndexMapping.Num());
	}
};

USTRUCT()
struct FNodeHierarchyWithUserData
{
	GENERATED_BODY()

protected:
	UPROPERTY()
	FNodeHierarchyData Hierarchy;

public: 
	virtual ~FNodeHierarchyWithUserData() = default;

	const FNodeHierarchyData& GetHierarchy() const { return Hierarchy; }
	const TArray<FTransform>& GetTransforms() const { return Hierarchy.Transforms; }
	TArray<FTransform>& GetTransforms() { return Hierarchy.Transforms; }
	const TArray<FNodeObject>& GetNodes() const { return Hierarchy.Nodes; }

	template<typename DataType>
	const DataType&	GetNodeData(int32 Index) const { return *reinterpret_cast<const DataType*>(GetUserDataImpl(Index)); }

	template<typename DataType>
	DataType& GetNodeData(int32 Index) { return *reinterpret_cast<DataType*>(GetUserDataImpl(Index)); }

	// it's up to your hierarchy to decide what to do with this
	virtual const FTransform& GetLocalTransform(int32 Index) const PURE_VIRTUAL(FNodeHierarchyWithUserData::GetLocalTransform, return Hierarchy.Transforms[Index];)
	virtual const FTransform& GetGlobalTransform(int32 Index) const PURE_VIRTUAL(FNodeHierarchyWithUserData::GetGlobalTransform, return Hierarchy.Transforms[Index];)
	virtual FTransform& GetLocalTransform(int32 Index) PURE_VIRTUAL(FNodeHierarchyWithUserData::GetLocalTransform, return Hierarchy.Transforms[Index];)
	virtual FTransform& GetGlobalTransform(int32 Index) PURE_VIRTUAL(FNodeHierarchyWithUserData::GetGlobalTransform, return Hierarchy.Transforms[Index];)
	virtual void SetLocalTransform(int32 Index, const FTransform& NewTransform) { }
	virtual void SetGlobalTransform(int32 Index, const FTransform& NewTransform) { }

	// get all list of children
	TArray<FName> GetChildren(int32 Index) const
	{
		TArray<FName> ChildrenNames;
		FName NodeName = Hierarchy.Nodes[Index].Name;
		for (int32 NodeIndex = 0; NodeIndex < Hierarchy.Nodes.Num(); ++NodeIndex)
		{
			if (Hierarchy.Nodes[NodeIndex].ParentName == NodeName)
			{
				ChildrenNames.Add(Hierarchy.Nodes[NodeIndex].Name);
			}
		}

		return ChildrenNames;
	}

	FTransform GetLocalTransformByName(const FName& NodeName) const
	{
		int32 Index = GetNodeIndex(NodeName);
		if (IsValidIndex(Index))
		{
			return GetLocalTransform(Index);
		}

		return FTransform::Identity;
	}

	FTransform GetGlobalTransformByName(const FName& NodeName) const
	{
		int32 Index = GetNodeIndex(NodeName);
		if (IsValidIndex(Index))
		{
			return GetGlobalTransform(Index);
		}

		return FTransform::Identity;
	}

	void SetLocalTransformByName(const FName& NodeName, const FTransform& NewTransform)
	{
		int32 Index = GetNodeIndex(NodeName);
		if (IsValidIndex(Index))
		{
			SetLocalTransform(Index, NewTransform);
		}
	}

	void SetGlobalTransformByName(const FName& NodeName, const FTransform& NewTransform)
	{
		int32 Index = GetNodeIndex(NodeName);
		if (IsValidIndex(Index))
		{
			SetGlobalTransform(Index, NewTransform);
		}
	}
	// should initialize all transient data for fast look up
	virtual void Initialize()
	{
		Hierarchy.BuildNodeNameToIndexMapping();
	}

	int32 GetParentIndex(int32 Index) const
	{
		return Hierarchy.GetParentIndex(Index);
	}

	FName GetParentName(int32 Index) const 
	{
		return Hierarchy.GetParentName(Index);
	}

	FName GetParentName(const FName& NodeName) const
	{
		int32 Index = GetNodeIndex(NodeName);
		if (IsValidIndex(Index))
		{
			return Hierarchy.GetParentName(Index);
		}

		return NAME_None;
	}

	void SetParentName(int32 Index, FName NewParent) 
	{
		// if no parent or if exist - reject typos
		if (NewParent == NAME_None || Contains(NewParent))
		{
			Hierarchy.SetParentName(Index, NewParent);
		}
	}

	int32 GetNodeIndex(const FName& InNodeName) const
	{
		return Hierarchy.GetNodeIndex(InNodeName);
	}

	FName GetNodeName(int32 Index) const
	{
		return Hierarchy.GetNodeName(Index);
	}

	void SetNodeName(int32 Index, const FName& NewNode)
	{
		Hierarchy.SetNodeName(Index, NewNode);
	}

	int32 Add(const FName& InNodeName, const FName& InParentName, const FTransform& InTransform)
	{
		check(!HasUserData());
		return Hierarchy.Add(InNodeName, InParentName, InTransform);
	}

	template<typename DataType>
	int32 Add(const FName& InNodeName, const FName& InParentName, const FTransform& InTransform, const DataType& InNodeData)
	{
		int32 Index = Hierarchy.Add(InNodeName, InParentName, InTransform);
		if (Index != INDEX_NONE && HasUserData())
		{
			int32 UserDataIndex = AddUserDataImpl(&InNodeData);

			check(UserDataIndex == Index);
			check(Hierarchy.Nodes.Num() == Hierarchy.Transforms.Num() && Hierarchy.Transforms.Num() == GetNumUserData());
		}

		return Index;
	}

	void Remove(const FName& InNodeName)
	{
		int32 RemovedIndex = Hierarchy.Remove(InNodeName);
		if (HasUserData())
		{
			if (RemovedIndex != INDEX_NONE)
			{
				RemoveUserData(RemovedIndex);
			}

			check(Hierarchy.Nodes.Num() == Hierarchy.Transforms.Num() && Hierarchy.Transforms.Num() == GetNumUserData());
		}
	}

	void Empty(int32 Size = 0)
	{
		Hierarchy.Empty(Size);
		if (HasUserData())
		{
			EmptyUserData(Size);

			check(Hierarchy.Nodes.Num() == Hierarchy.Transforms.Num() && Hierarchy.Transforms.Num() == GetNumUserData());
		}
	}

	int32 GetNum() const
	{
		return Hierarchy.Num();
	}

	bool IsValidIndex(int32 Index) const
	{
		return Hierarchy.IsValidIndex(Index);
	}

	bool Contains(const FName InNodeName) const
	{
		return GetNodeIndex(InNodeName) != INDEX_NONE;
	}

protected:
	/** Derived classes can implement this to supply per-node user data */
	virtual const void* GetUserDataImpl(int32 Index) const { return nullptr; }
	virtual void* GetUserDataImpl(int32 Index) { return nullptr; }
	virtual int32 AddUserDataImpl(const void* InData) { return INDEX_NONE; }
	virtual int32 GetNumUserData() const { return 0; }
	virtual void EmptyUserData(int32 Size = 0) {}
	virtual void RemoveUserData(int32 Index) {}
	virtual bool HasUserData() const { return false; }
};

template<>
struct TStructOpsTypeTraits<FNodeHierarchyWithUserData> : public TStructOpsTypeTraitsBase2<FNodeHierarchyWithUserData>
{
	enum
	{
		WithPureVirtual = true,
	};
};
