// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/MathFwd.h"

namespace AnimationCore
{
	/**
	 * Soft IK
	 *
	 * This modifies the effector position for a chain of joints being controlled by IK by making the effector softly approach full extension.
	 * It takes the desired effector position and adjusts it according to an exponential falloff as the effector approach the full length of the chain.
	 *
	 * The technique used is described here: https://web.archive.org/web/20160610183037/https://softimageblog.com/archives/108
	 *
	 * @param RootLocation			The location of the bone at the root of the chain
	 * @param TotalChainLength		The total length of the chain from root to tip, along the bones (not straight line).
	 * @param SoftLengthPercent		The percentage of the chain length to begin softening the effector motion (typically set to 0.97)
	 * @param Alpha					The amount of softness to apply (0 is none, 1 is full amount). Default is 1.
	 * @param InOutEffectorPosition	The input position of the end effector. This is modified by this function to "soften" the IK.
	 */
	ANIMATIONCORE_API void SoftenIKEffectorPosition(
		const FVector& RootLocation,
		const float TotalChainLength,
		const float SoftLengthPercent,
		const float Alpha,
		FVector& InOutEffectorPosition);
}
