// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Math/Axis.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"

struct FInterpCurveFloat;
struct FInterpCurveQuat;
struct FInterpCurveVector;

/** Outputs a float given a float input */
DECLARE_DELEGATE_RetVal_OneParam(float, FFloatMapping, float);

/**
 * Finds a spline parameter (alpha) at the intersection of a spline and a sphere. Used by SolveSplineIK.
 * The assumption is that the sphere's origin lies on the spline and the StartingLinearIndex is used to 
 * skip ahead when using a linear approximation to the spline.
 * @param	InOriginOnSpline			Sphere origin on the spline
 * @param	InRadius					Sphere radius
 * @param	InOutStartingLinearIndex	(inout) The index of the linear approximation to start at. After the function returns this should be set to a value that can be passed into the next call to accelerate subsequent queries.
 */
DECLARE_DELEGATE_RetVal_ThreeParams(float, FFindParamAtFirstSphereIntersection, const FVector& /*InOriginOnSpline*/, float /*InRadius*/, int32& /*InOutStartingLinearIndex*/)

namespace AnimationCore
{
	/**
	 * Spline IK
	 */
	ANIMATIONCORE_API void SolveSplineIK(const TArray<FTransform>& BoneTransforms, const FInterpCurveVector& PositionSpline, const FInterpCurveQuat& RotationSpline, const FInterpCurveVector& ScaleSpline, const float TotalSplineAlpha, const float TotalSplineLength, const FFloatMapping& Twist, const float Roll, const float Stretch, const float Offset, const EAxis::Type BoneAxis, const FFindParamAtFirstSphereIntersection& FindParamAtFirstSphereIntersection, const TArray<FQuat>& BoneOffsetRotations, const TArray<float>& BoneLengths, const float OriginalSplineLength, TArray<FTransform>& OutBoneTransforms);
};
