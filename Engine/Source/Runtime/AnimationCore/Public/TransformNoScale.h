// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Math/MathFwd.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "UObject/ObjectMacros.h"

#include "TransformNoScale.generated.h"

USTRUCT(BlueprintType)
struct FTransformNoScale
{
	GENERATED_BODY()

	/**
	 * The identity transformation (Rotation = FRotator::ZeroRotator, Translation = FVector::ZeroVector, Scale = (1,1,1)).
	 */
	static ANIMATIONCORE_API const FTransformNoScale Identity;

	inline FTransformNoScale()
		: Location(ForceInitToZero)
		, Rotation(ForceInitToZero)
	{
	}

	inline FTransformNoScale(const FVector& InLocation, const FQuat& InRotation)
		: Location(InLocation)
		, Rotation(InRotation)
	{
	}

	inline FTransformNoScale(const FTransform& InTransform)
		: Location(InTransform.GetLocation())
		, Rotation(InTransform.GetRotation())
	{
	}

	inline FTransformNoScale& operator =(const FTransform& InTransform)
	{
		FromFTransform(InTransform);
		return *this;
	}

	inline operator FTransform() const
	{
		return ToFTransform();
	}

	/** The translation of this transform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Transform")
	FVector Location;

	/** The rotation of this transform */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Transform")
	FQuat Rotation;

	/** Convert to an FTransform */
	inline FTransform ToFTransform() const
	{
		return FTransform(Rotation, Location, FVector::OneVector);
	}

	/** Convert from an FTransform */
	inline void FromFTransform(const FTransform& InTransform)
	{
		Location = InTransform.GetLocation();
		Rotation = InTransform.GetRotation();
	}
};
