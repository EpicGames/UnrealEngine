// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/MathFwd.h"
#include "Math/Transform.h"

namespace AnimationCore
{
	/**
	 * Two Bone IK
	 *
	 * This handles two bone chain link excluding root bone. This will solve the solution for joint/end when given root, joint, end position (root->joint->end in the hierarchy)
	 * based on effector, joint target location
	 * This only solves for location, if you want to rotate them to face target, this doesn't do it for you
	 *
	 * @param	RootPos				Root position
	 * @param	JointPos			Joint position
	 * @param	EndPos				End position
	 * @param	JointTarget			Joint Target position (where joint is facing while creating plane between joint pos, joint target, root pos(rotate-plane ik))
	 * @param	Effector			Effector position (target position)
	 * @param	OutJointPos			(out) adjusted joint pos
	 * @param	OutEndPos			(out) adjusted end pos
	 * @param	bAllowStretching	whether or not to allow stretching or not
	 * @param	StartStretchRatio	When should it start stretch -i.e. 1 means its own length without any stretch
	 * @param	MaxStretchScale		How much it can stretch to in ratio
	 */
	ANIMATIONCORE_API void SolveTwoBoneIK(const FVector& RootPos, const FVector& JointPos, const FVector& EndPos, const FVector& JointTarget, const FVector& Effector, FVector& OutJointPos, FVector& OutEndPos, bool bAllowStretching, double StartStretchRatio, double MaxStretchScale);

	/**
	* Two Bone IK
	*
	* This handles two bone chain link excluding root bone. This will solve the solution for joint/end when given root, joint, end position (root->joint->end in the hierarchy)
	* based on effector, joint target location
	* This only solves for location, if you want to rotate them to face target, this doesn't do it for you
	*
	* @param	RootPos				Root position
	* @param	JointPos			Joint position
	* @param	EndPos				End position
	* @param	JointTarget			Joint Target position (where joint is facing while creating plane between joint pos, joint target, root pos(rotate-plane ik))
	* @param	Effector			Effector position (target position)
	* @param	OutJointPos			(out) adjusted joint pos
	* @param	OutEndPos			(out) adjusted end pos
	* @param	bAllowStretching	whether or not to allow stretching or not								  O
	* @param	StartStretchRatio	When should it start stretch -i.e. 1 means its own length without any stretch
	* @param	MaxStretchScale		How much it can stretch to in ratio
	*/
	ANIMATIONCORE_API void SolveTwoBoneIK(const FVector& RootPos, const FVector& JointPos, const FVector& EndPos, const FVector& JointTarget, const FVector& Effector, FVector& OutJointPos, FVector& OutEndPos, double UpperLimbLength, double LowerLimbLength, bool bAllowStretching, double StartStretchRatio, double MaxStretchScale);

	/**
	* Two Bone IK
	*
	* This handles two bone chain link excluding root bone. This will solve the solution for joint/end when given root, joint, end position (root->joint->end in the hierarchy)
	* based on effector, joint target location
	* This only solves for location, if you want to rotate them to face target, this doesn't do it for you
	*
	* @param	RootPos				Root position
	* @param	JointPos			Joint position
	* @param	EndPos				End position
	* @param	JointTarget			Joint Target position (where joint is facing while creating plane between joint pos, joint target, root pos(rotate-plane ik))
	* @param	Effector			Effector position (target position)
	* @param	OutJointPos			(out) adjusted joint pos
	* @param	OutEndPos			(out) adjusted end pos
	* @param	bAllowStretching	whether or not to allow stretching or not
	* @param	StartStretchRatio	When should it start stretch -i.e. 1 means its own length without any stretch
	* @param	MaxStretchScale		How much it can stretch to in ratio
	*/
	ANIMATIONCORE_API void SolveTwoBoneIK(FTransform& InOutRootTransform, FTransform& InOutJointTransform, FTransform& InOutEndTransform, const FVector& JointTarget, const FVector& Effector, bool bAllowStretching, double StartStretchRatio, double MaxStretchScale);

	/**
	* Two Bone IK
	*
	* This handles two bone chain link excluding root bone. This will solve the solution for joint/end when given root, joint, end position (root->joint->end in the hierarchy)
	* based on effector, joint target location
	* This only solves for location, if you want to rotate them to face target, this doesn't do it for you
	*
	* @param	RootPos				Root position
	* @param	JointPos			Joint position
	* @param	EndPos				End position
	* @param	JointTarget			Joint Target position (where joint is facing while creating plane between joint pos, joint target, root pos(rotate-plane ik))
	* @param	Effector			Effector position (target position)
	* @param	OutJointPos			(out) adjusted joint pos
	* @param	OutEndPos			(out) adjusted end pos
	* @param	bAllowStretching	whether or not to allow stretching or not
	* @param	StartStretchRatio	When should it start stretch -i.e. 1 means its own length without any stretch
	* @param	MaxStretchScale		How much it can stretch to in ratio
	*/
	ANIMATIONCORE_API void SolveTwoBoneIK(FTransform& InOutRootTransform, FTransform& InOutJointTransform, FTransform& InOutEndTransform, const FVector& JointTarget, const FVector& Effector, double UpperLimbLength, double LowerLimbLength, bool bAllowStretching, double StartStretchRatio, double MaxStretchScale);
};
